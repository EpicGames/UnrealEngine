// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocketSubsystemAndroid.h"
#include "SocketSubsystemModule.h"
#include "SocketsAndroid.h"
#include "IPAddress.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "Modules/ModuleManager.h"
#include "BSDSockets/IPAddressBSD.h"

#include <sys/ioctl.h>
#include <net/if.h>

FSocketSubsystemAndroid* FSocketSubsystemAndroid::SocketSingleton = NULL;

FName CreateSocketSubsystem( FSocketSubsystemModule& SocketSubsystemModule )
{
	FName SubsystemName("ANDROID");
	// Create and register our singleton factor with the main online subsystem for easy access
	FSocketSubsystemAndroid* SocketSubsystem = FSocketSubsystemAndroid::Create();
	FString Error;
	if (SocketSubsystem->Init(Error))
	{
		SocketSubsystemModule.RegisterSocketSubsystem(SubsystemName, SocketSubsystem);
		return SubsystemName;
	}
	else
	{
		FSocketSubsystemAndroid::Destroy();
		return NAME_None;
	}
}

void DestroySocketSubsystem( FSocketSubsystemModule& SocketSubsystemModule )
{
	SocketSubsystemModule.UnregisterSocketSubsystem(FName("ANDROID"));
	FSocketSubsystemAndroid::Destroy();
}

/**
 * Create a FSocketBSD sub class capable of acquiring WifiManager.MulticastLock if needed.
 */
FSocketBSD* FSocketSubsystemAndroid::InternalBSDSocketFactory( SOCKET Socket, ESocketType SocketType, const FString& SocketDescription, const FName& SocketProtocol)
{
	// return a new socket object
	return new FSocketAndroid(Socket, SocketType, SocketDescription, SocketProtocol, this);
}

/** 
 * Singleton interface for the Android socket subsystem 
 * @return the only instance of the Android socket subsystem
 */
FSocketSubsystemAndroid* FSocketSubsystemAndroid::Create()
{
	if (SocketSingleton == NULL)
	{
		SocketSingleton = new FSocketSubsystemAndroid();
	}

	return SocketSingleton;
}

/** 
 * Destroy the singleton Android socket subsystem
 */
void FSocketSubsystemAndroid::Destroy()
{
	if (SocketSingleton != NULL)
	{
		SocketSingleton->Shutdown();
		delete SocketSingleton;
		SocketSingleton = NULL;
	}
}

/**
 * Does Android platform initialization of the sockets library
 *
 * @param Error a string that is filled with error information
 *
 * @return TRUE if initialized ok, FALSE otherwise
 */
bool FSocketSubsystemAndroid::Init(FString& Error)
{
	return true;
}

/**
 * Performs Android specific socket clean up
 */
void FSocketSubsystemAndroid::Shutdown(void)
{
}


/**
 * @return Whether the device has a properly configured network device or not
 */
bool FSocketSubsystemAndroid::HasNetworkDevice()
{
	return true;
}


/**
* @return Label explicitly as Android due to differences in how the kernel handles addresses
*/
const TCHAR* FSocketSubsystemAndroid::GetSocketAPIName() const
{
	return TEXT("BSD_Android");
}

/**
 * Creates a socket and ensures IPv6 sockets are dual-stack capable (IPV6_V6ONLY=false).
 * This matches the behavior of iOS, Unix, and Windows socket subsystems, allowing a
 * single IPv6 socket to accept both IPv4 and IPv6 connections.
 */
FSocket* FSocketSubsystemAndroid::CreateSocket(const FName& SocketType, const FString& SocketDescription, const FName& ProtocolType)
{
	FSocketBSD* NewSocket = (FSocketBSD*)FSocketSubsystemBSD::CreateSocket(SocketType, SocketDescription, ProtocolType);
	if (NewSocket != nullptr)
	{
		NewSocket->SetIPv6Only(false);
	}
	else
	{
		UE_LOGF(LogSockets, Warning, "Failed to create socket %ls [%ls]", *SocketType.ToString(), *SocketDescription);
	}
	return NewSocket;
}

/** Priority values for Android adapters. This code is copied from the iOS system as well. */
enum class EAdapterPriorityValues : uint8
{
	None = 0,
	Wifi = 1,
	Cell = 2,
	Alternative = 3
};

/** Helper struct to sort our adapters based off of interface information */
struct FSortedPriorityAddresses
{
	FSortedPriorityAddresses(FSocketSubsystemAndroid* SocketSub) :
		Priority(EAdapterPriorityValues::None)
	{
		Address = StaticCastSharedRef<FInternetAddrBSD>(SocketSub->CreateInternetAddr());
	}

	TSharedPtr<FInternetAddrBSD> Address;
	EAdapterPriorityValues Priority;

	bool operator<(const FSortedPriorityAddresses& Other) const
	{
		return Priority < Other.Priority;
	}

	FString ToString() const
	{
		FString AdapterPriorityValue;
		switch (Priority)
		{
		default:
		case EAdapterPriorityValues::None:
			AdapterPriorityValue = TEXT("Invalid/None");
			break;
		case EAdapterPriorityValues::Wifi:
			AdapterPriorityValue = TEXT("Wifi");
			break;
		case EAdapterPriorityValues::Cell:
			AdapterPriorityValue = TEXT("Cell");
			break;
		case EAdapterPriorityValues::Alternative:
			AdapterPriorityValue = TEXT("Alternative");
			break;
		}

		return FString::Printf(TEXT("%s %s (%d)"), *Address->ToString(true), *AdapterPriorityValue, (int32)Priority);
	}
};

bool FSocketSubsystemAndroid::GetLocalAdapterAddresses(TArray<TSharedPtr<FInternetAddr>>& OutAddresses)
{
	bool bSuccess = false;
	// Sorted address array. Android selection picking is based off the iOS picker as well.
	TArray<FSortedPriorityAddresses> SortedAddresses;

	// Due to the Android versions we target, things such as common network extensions like ifaddrs 
	// or various network scope queries just do not exist.
	// Because of this, we have to do several workarounds in order to work on all Android platforms.
	// This is really silly, but that's just how the kernel is.
	int TempSocket = socket(PF_INET, SOCK_STREAM, 0);
	if (TempSocket)
	{
		ifreq IfReqs[8];

		ifconf IfConfig;
		FMemory::Memzero(IfConfig);
		IfConfig.ifc_req = IfReqs;
		IfConfig.ifc_len = sizeof(IfReqs);

		int Result = ioctl(TempSocket, SIOCGIFCONF, &IfConfig);
		if (Result == 0)
		{
			// Temporary cache of the address we get per interface
			sockaddr_storage TemporaryAddress;

			// Grow the size of the TArray to make it so we don't have to grow on every addition
			const uint32 NumInterfaces = UE_ARRAY_COUNT(IfReqs);
			SortedAddresses.Reserve(NumInterfaces);

			for (int32 IdxReq = 0; IdxReq < NumInterfaces; ++IdxReq)
			{
				// Clear this out per loop.
				FMemory::Memzero(&TemporaryAddress, sizeof(sockaddr_storage));

				// Cache the address information, as the following flag lookup will 
				// write into the ifr_addr field.
				FMemory::Memcpy((void*)&TemporaryAddress, (void*)(&IfReqs[IdxReq].ifr_addr), sizeof(sockaddr_in));

				// Examine interfaces that are up and not loop back
				if (ioctl(TempSocket, SIOCGIFFLAGS, &IfReqs[IdxReq]) == 0 &&
					(IfReqs[IdxReq].ifr_flags & IFF_UP) &&
					(IfReqs[IdxReq].ifr_flags & IFF_LOOPBACK) == 0 &&
					TemporaryAddress.ss_family != AF_UNSPEC)
				{
					FSortedPriorityAddresses NewPriorityAddress(this);
					NewPriorityAddress.Address->SetIp(TemporaryAddress);

					if (strcmp(IfReqs[IdxReq].ifr_name, "wlan0") == 0)
					{
						NewPriorityAddress.Priority = EAdapterPriorityValues::Wifi;
					}
					else if (strcmp(IfReqs[IdxReq].ifr_name, "rmnet0") == 0)
					{
						NewPriorityAddress.Priority = EAdapterPriorityValues::Cell;
					}
					else if(TemporaryAddress.ss_family != AF_UNSPEC)
					{
						NewPriorityAddress.Priority = EAdapterPriorityValues::Alternative;
					}

					SortedAddresses.Add(NewPriorityAddress);
				}
			}

			// Sort the array so that we have the correct priorities straight
			SortedAddresses.Sort();

			// Add the sorted addresses to our output array.
			for (const auto& SortedAddress : SortedAddresses)
			{
				OutAddresses.Add(SortedAddress.Address);
				UE_LOGF(LogSockets, Log, "(%ls) Address %ls", GetSocketAPIName(), *SortedAddress.ToString());
			}
			bSuccess = true;
		}
		else
		{
			int ErrNo = errno;
			UE_LOGF(LogSockets, Warning, "ioctl( ,SIOGCIFCONF, ) failed, errno=%d (%ls)", ErrNo, ANSI_TO_TCHAR(strerror(ErrNo)));
			bSuccess = false;
		}

		close(TempSocket);
	}

	return bSuccess;
}

/**
 * Translates an ESocketAddressInfoFlags into a value usable by getaddrinfo
 */
int32 FSocketSubsystemAndroid::GetAddressInfoHintFlag(EAddressInfoFlags InFlags) const
{
	// On Android, you cannot explicitly use AI_ALL and AI_V4MAPPED
	// However, if GetAddressInfo is passed with no hint flags, the query will execute with
	// AI_V4MAPPED automatically (flag can only be set by the kernel)
	EAddressInfoFlags ModifiedInFlags = (InFlags & ~EAddressInfoFlags::AllResultsWithMapping);
	return FSocketSubsystemBSD::GetAddressInfoHintFlag(ModifiedInFlags);
}
