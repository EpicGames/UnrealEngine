// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SocketSubsystem.h"
#include "BSDSockets/SocketSubsystemBSD.h"
#include "SocketSubsystemPackage.h"

/**
 * Android specific socket subsystem implementation
 */
class FSocketSubsystemAndroid : public FSocketSubsystemBSD
{
protected:

	/** Single instantiation of this subsystem */
	static FSocketSubsystemAndroid* SocketSingleton;

	/** Whether Init() has been called before or not */
	bool bTriedToInit;

	/**
	 * Create a FSocketBSD sub class capable of acquiring WifiManager.MulticastLock if needed.
	 */
	virtual class FSocketBSD* InternalBSDSocketFactory( SOCKET Socket, ESocketType SocketType, const FString& SocketDescription, const FName& SocketProtocol) override;

	// @todo android: (inherited from iOS) This is kind of hacky, since there's no UBT that should set PACKAGE_SCOPE
// PACKAGE_SCOPE:
public:

	/** 
	 * Singleton interface for this subsystem 
	 * @return the only instance of this subsystem
	 */
	static FSocketSubsystemAndroid* Create();

	/**
	 * Performs Android specific socket clean up
	 */
	static void Destroy();

public:

	FSocketSubsystemAndroid() 
		: bTriedToInit(false)
	{
	}

	virtual ~FSocketSubsystemAndroid() = default;

	/**
	 * Does Android platform initialization of the sockets library
	 *
	 * @param Error a string that is filled with error information
	 *
	 * @return TRUE if initialized ok, FALSE otherwise
	 */
	virtual bool Init(FString& Error) override;

	/**
	 * Performs platform specific socket clean up
	 */
	virtual void Shutdown() override;

	/**
	 * @return Whether the device has a properly configured network device or not
	 */
	virtual bool HasNetworkDevice() override;

	/**
	 * Get the name of the socket subsystem
	 * @return a string naming this subsystem
	 */
	virtual const TCHAR* GetSocketAPIName() const override;

	/**
	 * Translates an ESocketAddressInfoFlags into a value usable by getaddrinfo
	 */

	virtual int32 GetAddressInfoHintFlag(EAddressInfoFlags InFlags) const override;

	virtual FSocket* CreateSocket(const FName& SocketType, const FString& SocketDescription, const FName& ProtocolType) override;

	virtual bool GetLocalAdapterAddresses(TArray<TSharedPtr<FInternetAddr>>& OutAddresses) override;

	virtual FName GetDefaultSocketProtocolFamily() const override
	{
		return FNetworkProtocolTypes::IPv4;
	}
};
