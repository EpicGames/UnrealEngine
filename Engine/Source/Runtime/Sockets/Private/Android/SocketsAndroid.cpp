// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocketsAndroid.h"

FSocketAndroid::~FSocketAndroid()
{
    ReleaseMulticastLock();
}

bool FSocketAndroid::SetBroadcast(bool bAllowBroadcast)
{
    bool bSucceeded = FSocketBSD::SetBroadcast(bAllowBroadcast);
    if (bSucceeded && bAllowBroadcast && !bIsMulticastLockAcquired)
    {
        AcquireMulticastLock();
    }
    return bSucceeded;
}

bool FSocketAndroid::JoinMulticastGroup(const FInternetAddr& GroupAddress)
{
    bool bShouldReleaseLockOnFailure = !bIsMulticastLockAcquired;
    if (bShouldReleaseLockOnFailure)
    {
        AcquireMulticastLock();
    } 
    bool bSucceeded = FSocketBSD::JoinMulticastGroup(GroupAddress);
    if (!bSucceeded && bShouldReleaseLockOnFailure)
    {
        ReleaseMulticastLock();
    }
    return bSucceeded;
}

bool FSocketAndroid::JoinMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress)
{
    bool bShouldReleaseLockOnFailure = !bIsMulticastLockAcquired;
    if (bShouldReleaseLockOnFailure)
    {
        AcquireMulticastLock();
    } 
    bool bSucceeded = FSocketBSD::JoinMulticastGroup(GroupAddress, InterfaceAddress);
    if (!bSucceeded && bShouldReleaseLockOnFailure)
    {
        ReleaseMulticastLock();
    }
    return bSucceeded;
}

void FSocketAndroid::AcquireMulticastLock()
{
#if USE_ANDROID_JNI
    extern bool AndroidThunkCpp_AcquireWifiManagerMulticastLock();
    if (AndroidThunkCpp_AcquireWifiManagerMulticastLock())
    {
        bIsMulticastLockAcquired = true;
		UE_LOGF(LogSockets, VeryVerbose, "WifiManager.MulticastLock succesfully aquired");
    }
    else
#endif
    {
        static bool bAlreadyLoggedFailure = false;  
        if (!bAlreadyLoggedFailure) 
        { 
            bAlreadyLoggedFailure = true;
    		UE_LOGF(LogSockets, Warning, "Failed to acquire WifiManager.MulticastLock. Is multicast/broadcast support for Android enabled in AndroidRuntimeSettings?");
        }
    }
}

void FSocketAndroid::ReleaseMulticastLock()
{
#if USE_ANDROID_JNI
    if (bIsMulticastLockAcquired)
    {
	    UE_LOGF(LogSockets, VeryVerbose, "Releasing WifiManager.MulticastLock");
        extern void AndroidThunkCpp_ReleaseWifiManagerMulticastLock();
        AndroidThunkCpp_ReleaseWifiManagerMulticastLock();
        bIsMulticastLockAcquired = false;
    }
#endif
}