// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BSDSockets/SocketsBSD.h"

class FSocketAndroid
	: public FSocketBSD
{
public:
    using FSocketBSD::FSocketBSD;

	virtual ~FSocketAndroid() override;

	// FSocket overrides already overriden by FSocketBSD
	// Those will wrap the FSocketBSD implementation and acquire the multicast lock if needed
    virtual bool SetBroadcast(bool bAllowBroadcast = true) override;
	virtual bool JoinMulticastGroup(const FInternetAddr& GroupAddress) override;
	virtual bool JoinMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress) override;

private:
	void AcquireMulticastLock();
	void ReleaseMulticastLock();

	bool bIsMulticastLockAcquired = false;
};