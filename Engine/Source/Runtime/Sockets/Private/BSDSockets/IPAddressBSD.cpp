// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPAddressBSD.h"

#include "Math/NumericLimits.h"
#include "SocketSubsystemBSD.h"
#include "AutoRTFM.h"

#if PLATFORM_HAS_BSD_SOCKETS || PLATFORM_HAS_BSD_IPV6_SOCKETS

#if PLATFORM_HAS_BSD_IPV6_SOCKETS
void MapIPv4ToIPv6(const uint32& InAddress, in6_addr& OutStructure)
{
	FMemory::Memzero(OutStructure);
	OutStructure.s6_addr[10] = 0xff;
	OutStructure.s6_addr[11] = 0xff;
	OutStructure.s6_addr[12] = (static_cast<uint32>(InAddress) & 0xFF);
	OutStructure.s6_addr[13] = ((static_cast<uint32>(InAddress) >> 8) & 0xFF);
	OutStructure.s6_addr[14] = ((static_cast<uint32>(InAddress) >> 16) & 0xFF);
	OutStructure.s6_addr[15] = ((static_cast<uint32>(InAddress) >> 24) & 0xFF);
}
#endif

FInternetAddrBSD::FInternetAddrBSD()
{
	SocketSubsystem = nullptr;
	Clear();
}

FInternetAddrBSD::FInternetAddrBSD(FSocketSubsystemBSD* InSocketSubsystem, FName RequestedProtocol) : 
	SocketSubsystem(InSocketSubsystem)
{
	Clear();

	if (RequestedProtocol.IsNone() && InSocketSubsystem)
	{
		RequestedProtocol = InSocketSubsystem->GetDefaultSocketProtocolFamily();
	}

	if (RequestedProtocol == FNetworkProtocolTypes::IPv4)
	{
		Addr.ss_family = AF_INET;
	}
#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	else if (RequestedProtocol == FNetworkProtocolTypes::IPv6)
	{
		Addr.ss_family = AF_INET6;
	}
#endif
	else
	{
		Addr.ss_family = AF_UNSPEC;
	}
}

bool FInternetAddrBSD::CompareEndpoints(const FInternetAddr& InAddr) const
{
	const FInternetAddrBSD& OtherBSD = static_cast<const FInternetAddrBSD&>(InAddr);
	if (GetPort() != OtherBSD.GetPort())
	{
		return false;
	}

	// If we share the same addresses, then just let the comparison operator take over.
	if (Addr.ss_family == OtherBSD.Addr.ss_family)
	{
		return *this == InAddr;
	}
	else if (Addr.ss_family == AF_INET || OtherBSD.Addr.ss_family == AF_INET)
	{
#if PLATFORM_HAS_BSD_IPV6_SOCKETS
		// To handle mapped addresses, we want to raise one of the addresses to IPv6 and then do the comparison.
		const in6_addr* IPv6Addr = (Addr.ss_family == AF_INET6) ? &((sockaddr_in6*)&Addr)->sin6_addr
			: &((sockaddr_in6*)&(OtherBSD.Addr))->sin6_addr;

		// Figure out which address is the one that needs to be raised to IPv6
		const in_addr* IPv4Addr = (Addr.ss_family == AF_INET) ? &((sockaddr_in*)&Addr)->sin_addr
			: &((sockaddr_in*)&(OtherBSD.Addr))->sin_addr;

		// Check special addresses first (Multicast, Any, Loopback)
		if ((IN6_IS_ADDR_MC_LINKLOCAL(IPv6Addr) && IN_MULTICAST(IPv4Addr->s_addr)) ||
			(IN6_IS_ADDR_UNSPECIFIED(IPv6Addr) && IPv4Addr->s_addr == INADDR_ANY) ||
			(IN6_IS_ADDR_LOOPBACK(IPv6Addr) && IPv4Addr->s_addr == INADDR_LOOPBACK))
		{
			return true;
		}

		// If we're not IPv4 mapped already, then we're not able to be compared
		// and should early out
		if (!IN6_IS_ADDR_V4MAPPED(IPv6Addr))
		{
			return false;
		}

		in6_addr ConvertedAddrData;
		MapIPv4ToIPv6(IPv4Addr->s_addr, ConvertedAddrData);
		return memcmp(&(ConvertedAddrData), IPv6Addr, sizeof(in6_addr)) == 0;
#else
		return false;
#endif
	}
	return false;
}

void FInternetAddrBSD::Clear()
{
	FMemory::Memzero(&Addr, sizeof(Addr));
	Addr.ss_family = AF_UNSPEC;
}

void FInternetAddrBSD::ResetScopeId()
{
#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	if (Addr.ss_family == AF_INET6)
	{
		((sockaddr_in6*)&Addr)->sin6_scope_id = 0;
	}
#endif
}

uint32 FInternetAddrBSD::GetScopeId() const
{
#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	if (Addr.ss_family == AF_INET6)
	{
		return ntohl(((sockaddr_in6*)&Addr)->sin6_scope_id);
	}
#endif
	return 0;
}

void FInternetAddrBSD::SetScopeId(uint32 NewScopeId)
{
#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	if (Addr.ss_family == AF_INET6)
	{
		((sockaddr_in6*)&Addr)->sin6_scope_id = htonl(NewScopeId);
	}
#endif
}

void FInternetAddrBSD::SetIp(const sockaddr_storage& IpAddr)
{
	// Instead of just replacing the structures entirely, we copy only the ip portion.
	// As this should not also set port.
	if (IpAddr.ss_family == AF_INET)
	{
		const sockaddr_in* SockAddr = (const sockaddr_in*)&IpAddr;
		SetIp(SockAddr->sin_addr);
	}

#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	if (IpAddr.ss_family == AF_INET6)
	{
		const sockaddr_in6* SockAddr = (const sockaddr_in6*)&IpAddr;
		SetIp(SockAddr->sin6_addr);
		// Remember to set the scope id.
		((sockaddr_in6*)&Addr)->sin6_scope_id = SockAddr->sin6_scope_id;
	}
#endif
}

void FInternetAddrBSD::SetIp(const TCHAR* InAddr, bool& bIsValid)
{
	bIsValid = false;
	FString AddressString(InAddr);
	FString Port;

	// Find some colons to try to determine the input given to us.
	const int32 FirstColonIndex = AddressString.Find(":", ESearchCase::IgnoreCase, ESearchDir::FromStart);
	const int32 LastColonIndex = AddressString.Find(":", ESearchCase::IgnoreCase, ESearchDir::FromEnd);

	// Either this is an IPv6 address as we will have a "]:" when ports are involved
	// or this is an IPv4 address as there's one colon and it exists in the string
	if (AddressString.Contains("]:") || (FirstColonIndex == LastColonIndex && LastColonIndex != INDEX_NONE))
	{
		Port = AddressString.RightChop(LastColonIndex + 1);
		AddressString.LeftInline(LastColonIndex, EAllowShrinking::No);
	}

	// Strip these for backwards compatibility.
	AddressString.RemoveFromStart("[");
	AddressString.RemoveFromEnd("]");
	
	if (SocketSubsystem != nullptr)
	{
		TSharedPtr<FInternetAddr> NewAddr = SocketSubsystem->GetAddressFromString(AddressString);
		if (NewAddr.IsValid())
		{
			bIsValid = true;
			SetRawIp(NewAddr->GetRawIp());
			if (!Port.IsEmpty())
			{
				SetPort(FCString::Atoi(*Port));
			}
		}
	}
	else
	{
		UE_LOGF(LogSockets, Verbose, "SocketSubsystem pointer is null, cannot resolve the stringed address");
	}
}

void FInternetAddrBSD::SetIp(uint32 InAddr)
{
#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	if (GetProtocolType() == FNetworkProtocolTypes::IPv6)
	{
		if (InAddr == 0)
		{
			SetAnyIPv6Address();
		}
		else if (InAddr == INADDR_BROADCAST)
		{
			SetIPv6BroadcastAddress();
		}
		else
		{
			in6_addr ConvertedAddrData;
			MapIPv4ToIPv6(htonl(InAddr), ConvertedAddrData);
			SetIp(ConvertedAddrData);
		}
		return;
	}
#endif

	((sockaddr_in*)&Addr)->sin_addr.s_addr = htonl(InAddr);
	Addr.ss_family = AF_INET;
}

void FInternetAddrBSD::Set(const sockaddr_storage& AddrData)
{
	Addr = AddrData;
}

void FInternetAddrBSD::Set(const sockaddr_storage& AddrData, SOCKLEN AddrLen)
{
	Clear();
	FMemory::Memcpy(&Addr, &AddrData, (size_t)AddrLen);
}

TArray<uint8> FInternetAddrBSD::GetRawIp() const
{
	TArray<uint8> RawAddressArray;
	if (Addr.ss_family == AF_INET)
	{
		const sockaddr_in* IPv4Addr = ((const sockaddr_in*)&Addr);
		uint32 IntAddr = IPv4Addr->sin_addr.s_addr;
		RawAddressArray.Add((IntAddr >> 0) & 0xFF);
		RawAddressArray.Add((IntAddr >> 8) & 0xFF);
		RawAddressArray.Add((IntAddr >> 16) & 0xFF);
		RawAddressArray.Add((IntAddr >> 24) & 0xFF);
	}
#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	else if (Addr.ss_family == AF_INET6)
	{
		const sockaddr_in6* IPv6Addr = ((const sockaddr_in6*)&Addr);
		for (int i = 0; i < 16; ++i)
		{
			RawAddressArray.Add(IPv6Addr->sin6_addr.s6_addr[i]);
		}
	}
#endif

	return RawAddressArray;
}

void FInternetAddrBSD::SetRawIp(const TArray<uint8>& RawAddr)
{
	if (RawAddr.Num() == 4) // This is IPv4
	{
		Addr.ss_family = AF_INET;
		sockaddr_in* IPv4Addr = ((sockaddr_in*)&Addr);
		IPv4Addr->sin_addr.s_addr =	(RawAddr[0] << 0) | (RawAddr[1] << 8) | (RawAddr[2] << 16) | (RawAddr[3] << 24);
	}
#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	else if(RawAddr.Num() >= 16) // We are IPv6
	{
		sockaddr_in6* IPv6Addr = ((sockaddr_in6*)&Addr);
		for (int i = 0; i < 16; ++i)
		{
			IPv6Addr->sin6_addr.s6_addr[i] = RawAddr[i];
		}

		Addr.ss_family = AF_INET6;
	}
#endif
	else
	{
		Clear();
	}
}

void FInternetAddrBSD::GetIp(uint32& OutAddr) const
{
	if (GetProtocolType() == FNetworkProtocolTypes::IPv6)
	{
		OutAddr = 0;

#if PLATFORM_HAS_BSD_IPV6_SOCKETS
		sockaddr_in6* IPv6Addr = ((sockaddr_in6*)&Addr);
		if (IN6_IS_ADDR_V4MAPPED(&IPv6Addr->sin6_addr))
		{
#if PLATFORM_LITTLE_ENDIAN
			OutAddr = (IPv6Addr->sin6_addr.s6_addr[12] << 24) | (IPv6Addr->sin6_addr.s6_addr[13] << 16) | (IPv6Addr->sin6_addr.s6_addr[14] << 8) | (IPv6Addr->sin6_addr.s6_addr[15]);
#else
			OutAddr = (IPv6Addr->sin6_addr.s6_addr[15] << 24) | (IPv6Addr->sin6_addr.s6_addr[14] << 16) | (IPv6Addr->sin6_addr.s6_addr[13] << 8) | (IPv6Addr->sin6_addr.s6_addr[12]);
#endif
		}
#endif
		return;
	}

	OutAddr = ntohl(((sockaddr_in*)&Addr)->sin_addr.s_addr);
}

void FInternetAddrBSD::SetPort(int32 InPort)
{
#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	if (GetProtocolType() == FNetworkProtocolTypes::IPv6)
	{
		((sockaddr_in6*)&Addr)->sin6_port = htons(IntCastChecked<uint16>(InPort));
		return;
	}
#endif

	((sockaddr_in*)&Addr)->sin_port = htons(IntCastChecked<uint16>(InPort));
}

/** Report whether the port is in a valid range for SetPort. */
bool FInternetAddrBSD::IsPortValid(int32 InPort) const
{
	return 0 <= InPort && InPort <= MAX_uint16;
}


int32 FInternetAddrBSD::GetPort() const
{
#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	if (GetProtocolType() == FNetworkProtocolTypes::IPv6)
	{
		return ntohs(((sockaddr_in6*)&Addr)->sin6_port);
	}
#endif

	return ntohs(((sockaddr_in*)&Addr)->sin_port);
}

void FInternetAddrBSD::SetAnyAddress()
{
	if (SocketSubsystem != nullptr)
	{
		if (GetProtocolType() == FNetworkProtocolTypes::IPv6)
		{
			SetAnyIPv6Address();
		}
		else
		{
			SetAnyIPv4Address();
		}
	}
	else
	{
		UE_LOGF(LogSockets, Warning, "Could not determine the default protocol to use in SetAnyAddress!");
	}
}

void FInternetAddrBSD::SetAnyIPv4Address()
{
	Clear();
	((sockaddr_in*)&Addr)->sin_addr.s_addr = htonl(INADDR_ANY);
	Addr.ss_family = AF_INET;
}

void FInternetAddrBSD::SetAnyIPv6Address()
{
	Clear();
#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	SetIp(in6addr_any);
#endif
}

void FInternetAddrBSD::SetBroadcastAddress()
{
	if (SocketSubsystem)
	{
		if (GetProtocolType() == FNetworkProtocolTypes::IPv6)
		{
			SetIPv6BroadcastAddress();
		}
		else
		{
			SetIPv4BroadcastAddress();
		}
	}
	else
	{
		UE_LOGF(LogSockets, Warning, "Could not determine the default protocol to use in SetBroadcastAddress!");
	}
}

void FInternetAddrBSD::SetIPv4BroadcastAddress()
{
	Clear();
	((sockaddr_in*)&Addr)->sin_addr.s_addr = htonl(INADDR_BROADCAST);
	Addr.ss_family = AF_INET;
}

void FInternetAddrBSD::SetIPv6BroadcastAddress()
{
	Clear();
#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	// broadcast means something different in IPv6, but this is a rough equivalent
#if !defined(in6addr_allnodesonlink) && !PLATFORM_MICROSOFT
		// see RFC 4291, link-local multicast address http://tools.ietf.org/html/rfc4291
	static in6_addr in6addr_allnodesonlink =
	{
		{ { 0xff, 0x02, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x01 } }
	};
#endif // in6addr_allnodesonlink
	SetIp(in6addr_allnodesonlink);
#endif
}

void FInternetAddrBSD::SetLoopbackAddress()
{
	if (SocketSubsystem)
	{
		if (GetProtocolType() == FNetworkProtocolTypes::IPv6)
		{
			SetIPv6LoopbackAddress();
		}
		else
		{
			SetIPv4LoopbackAddress();
		}
	}
	else
	{
		UE_LOGF(LogSockets, Warning, "Could not determine the default protocol to use in SetLoopbackAddress!");
	}
}

void FInternetAddrBSD::SetIPv4LoopbackAddress()
{
	Clear();
	((sockaddr_in*)&Addr)->sin_addr.s_addr = htonl(INADDR_LOOPBACK);
	Addr.ss_family = AF_INET;
}

void FInternetAddrBSD::SetIPv6LoopbackAddress()
{
	Clear();
#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	SetIp(in6addr_loopback);
#endif
}

FString FInternetAddrBSD::ToString(bool bAppendPort) const
{
	return AutoRTFM::Open([&]
	{
		FString ReturnVal;
#if PLATFORM_HAS_BSD_SOCKET_FEATURE_GETNAMEINFO
		char IPStr[NI_MAXHOST];
		if (getnameinfo((const sockaddr*)&Addr, GetStorageSize(), IPStr, NI_MAXHOST, nullptr, 0, NI_NUMERICHOST) == 0)
		{
			if (GetProtocolType() == FNetworkProtocolTypes::IPv6)
			{
				FString IPv6Str(ANSI_TO_TCHAR(IPStr));
				// Remove the scope interface if it exists.
				const int32 InterfaceMarkerIndex = IPv6Str.Find("%", ESearchCase::CaseSensitive, ESearchDir::FromEnd);
				if (InterfaceMarkerIndex != INDEX_NONE)
				{
					IPv6Str.LeftInline(InterfaceMarkerIndex, EAllowShrinking::No);
				}

				// Using dynamic formatting strings are deprecated.
				ReturnVal = bAppendPort ? FString::Printf(TEXT("[%s]"), *IPv6Str) 
					: FString::Printf(TEXT("%s"), *IPv6Str);
			}
			else
			{
				ReturnVal = IPStr;
			}

			if (bAppendPort)
			{
				ReturnVal += FString::Printf(TEXT(":%d"), GetPort());
			}
		}
#else
		UE_LOGF(LogSockets, Error, "Platform has no getnameinfo(), but did not override FInternetAddrBSD::ToString()");
#endif
		return ReturnVal;
	});
}

bool FInternetAddrBSD::operator==(const FInternetAddr& Other) const
{
	const FInternetAddrBSD& OtherBSD = static_cast<const FInternetAddrBSD&>(Other);
	FName CurrentFamily = GetProtocolType();

	// Check if the addr families match
	if (OtherBSD.GetProtocolType() != CurrentFamily)
	{
		return false;
	}

	// If the ports don't match, already fail out.
	if (GetPort() != OtherBSD.GetPort())
	{
		return false;
	}

#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	if (CurrentFamily == FNetworkProtocolTypes::IPv6)
	{
		const sockaddr_in6* OtherBSDAddr = (sockaddr_in6*)&(OtherBSD.Addr);
		const sockaddr_in6* ThisBSDAddr = ((sockaddr_in6*)&Addr);
		return memcmp(&(ThisBSDAddr->sin6_addr), &(OtherBSDAddr->sin6_addr), sizeof(in6_addr)) == 0;
	}
#endif

	if (CurrentFamily == FNetworkProtocolTypes::IPv4)
	{
		const sockaddr_in* OtherBSDAddr = (sockaddr_in*)&(OtherBSD.Addr);
		const sockaddr_in* ThisBSDAddr = ((sockaddr_in*)&Addr);
		return ThisBSDAddr->sin_addr.s_addr == OtherBSDAddr->sin_addr.s_addr;
	}

	return false;
}

bool FInternetAddrBSD::IsValid() const
{
	FName CurrentFamily = GetProtocolType();

	if (CurrentFamily == FNetworkProtocolTypes::IPv4)
	{
		return ((sockaddr_in*)&Addr)->sin_addr.s_addr != 0;
	}

#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	if (CurrentFamily == FNetworkProtocolTypes::IPv6)
	{
		in6_addr EmptyAddr;
		FMemory::Memzero(EmptyAddr);

		sockaddr_in6* ThisBSDAddr = ((sockaddr_in6*)&Addr);
		return memcmp(&(ThisBSDAddr->sin6_addr), &EmptyAddr, sizeof(in6_addr)) != 0;
	}
#endif

	return false;
}

TSharedRef<FInternetAddr> FInternetAddrBSD::Clone() const
{
	TSharedRef<FInternetAddrBSD> NewAddress = MakeShareable(new FInternetAddrBSD(SocketSubsystem, GetProtocolType()));
	NewAddress->Addr = Addr;
	return NewAddress;
}

FName FInternetAddrBSD::GetProtocolType() const
{
	switch (Addr.ss_family)
	{
		case AF_INET:
			return FNetworkProtocolTypes::IPv4;
		case AF_INET6:
			return FNetworkProtocolTypes::IPv6;
		default:
			return NAME_None;
	}
}

SOCKLEN FInternetAddrBSD::GetStorageSize() const
{
	if (GetProtocolType() == FNetworkProtocolTypes::IPv4)
	{
		return sizeof(sockaddr_in);
	}

#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	return sizeof(sockaddr_in6);
#else
	return sizeof(sockaddr_storage);
#endif
}

uint32 FInternetAddrBSD::GetTypeHash() const
{
	FName CurrentFamily = GetProtocolType();

	if (CurrentFamily == FNetworkProtocolTypes::IPv4)
	{
		return ntohl(((sockaddr_in*)&Addr)->sin_addr.s_addr) + (GetPort() * 23);
	}
	else if (CurrentFamily == FNetworkProtocolTypes::IPv6)
	{
		return FCrc::Strihash_DEPRECATED(*ToString(true));
	}

	return 0;
}

#endif
