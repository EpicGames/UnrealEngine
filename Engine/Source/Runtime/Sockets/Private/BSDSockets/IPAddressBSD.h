// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BSDSockets/SocketSubsystemBSDPrivate.h"
#include "IPAddress.h"
#include "SocketTypes.h"
#include "SocketSubsystemPackage.h"

#if PLATFORM_HAS_BSD_SOCKETS || PLATFORM_HAS_BSD_IPV6_SOCKETS

class FSocketSubsystemBSD;

/**
 * Represents an internet ip address with support for ipv4/v6. 
 * All data is in network byte order
 */
class FInternetAddrBSD : public FInternetAddr
{
	/** The internet ip address structure */
	sockaddr_storage Addr;

protected:
	/** The Subsystem that created this address */
	FSocketSubsystemBSD* SocketSubsystem;

	void Clear();
	void ResetScopeId();

PACKAGE_SCOPE:

	/**
	 * Sets the ip address using a network byte order ipv4 address
	 *
	 * @param IPv4Addr the new ip address to use
	 */
	virtual void SetIp(const in_addr& IPv4Addr)
	{
		((sockaddr_in*)&Addr)->sin_addr = IPv4Addr;
		Addr.ss_family = AF_INET;
	}

#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	/**
	 * Sets the ip address using a network byte order ipv6 address
	 *
	 * @param IpAddr the new ip address to use
	 */
	virtual void SetIp(const in6_addr& IPv6Addr)
	{
		((sockaddr_in6*)&Addr)->sin6_addr = IPv6Addr;
		Addr.ss_family = AF_INET6;
	}
#endif

public:
	/**
	 * Constructor. Sets address to default state
	 */
	FInternetAddrBSD();

	FInternetAddrBSD(FSocketSubsystemBSD* InSocketSubsystem, FName RequestedProtocol=NAME_None);

	/**
	 * Compares FInternetAddrs together, comparing the logical net addresses (endpoints)
	 * of the data stored, rather than doing a memory comparison like the equality operator does.
	 *
	 * @Param InAddr The address to compare with.
	 *
	 * @return true if the endpoint stored in this FInternetAddr is the same as the input.
	 */
	virtual bool CompareEndpoints(const FInternetAddr& InAddr) const override;

	/**
	 * Sets the ip address from a raw network byte order array.
	 *
	 * @param RawAddr the new address to use (must be converted to network byte order)
	 */
	virtual void SetRawIp(const TArray<uint8>& RawAddr) override;

	/**
	 * Gets the ip address in a raw array stored in network byte order.
	 *
	 * @return The raw address stored in an array
	 */
	virtual TArray<uint8> GetRawIp() const override;

	/**
	 * Sets the ip address from a host byte order uint32.
	 * Will attempt to determine platform compatibility for best way to store address
	 *
	 * @param InAddr the new address to use (must convert to network byte order)
	 */
	virtual void SetIp(uint32 InAddr) override;

	/**
	 * Sets the ip address from a string IPv6 or IPv4 address.
	 * Ports may be included using the form Address:Port, or excluded and set manually.
	 *
	 * IPv6 - [1111:2222:3333:4444:5555:6666:7777:8888]:PORT || [1111:2222:3333::]:PORT || [::ffff:IPv4]:PORT || any of these without the brackets and port
	 * IPv4 - aaa.bbb.ccc.ddd:PORT || 127.0.0.1:PORT
	 *
	 * @param InAddr the string containing the new ip address to use
	 * @param bIsValid will be set to true if InAddr was a valid IPv6 or IPv4 address, false if not.
	 */
	virtual void SetIp(const TCHAR* InAddr, bool& bIsValid) override;

	/**
	 * Sets the ip address using a generic sockaddr_storage
	 *
	 * @param IpAddr the new ip address to use (assumes already in the correct byte order).
	 */
	void SetIp(const sockaddr_storage& IpAddr);

	/**
	 * Sets the address data via a sockaddr_storage
	 * Can only really safely set with a sockaddr_storage
	 *
	 * @param AddrData the new data to use
	 */
	virtual void Set(const sockaddr_storage& AddrData);

	/**
	 * Sets the address data via a sockaddr_storage.
	 * Uses memcpy to assign the data.
	 *
	 * @param AddrData the new data to use
	 * @param AddrSize size of the addr data being passed in.
	 */
	virtual void Set(const sockaddr_storage& AddrData, SOCKLEN AddrSize);

#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	/**
	 * Copies the network byte order ip address
	 *
	 * @param OutAddr the out param receiving the ip address
	 */
	void GetIp(in6_addr& OutAddr) const
	{
		if (GetProtocolType() != FNetworkProtocolTypes::IPv6)
		{
			return;
		}

		OutAddr = ((sockaddr_in6*)&Addr)->sin6_addr;
	}
#endif

	/**
	 * Copies the network byte order ip address
	 *
	 * @param OutAddr the out param receiving the ip address
	 */
	void GetIp(in_addr& OutAddr) const
	{
		if (GetProtocolType() != FNetworkProtocolTypes::IPv4)
		{
			return;
		}

		OutAddr = ((sockaddr_in*)&Addr)->sin_addr;
	}

	/**
	 * Copies the network byte order ip address to a host byte order dword.
	 * Does nothing if we are currently not storing an ipv4 addr in some capacity
	 *
	 * @param OutAddr the out param receiving the ip address
	 */
	virtual void GetIp(uint32& OutAddr) const override;

	sockaddr_storage* GetRawAddr()
	{
		return &Addr;
	}

	/**
	 * Sets the port number from a host byte order int
	 *
	 * @param InPort the new port to use (must convert to network byte order)
	 */
	virtual void SetPort(int32 InPort) override;

	/** Report whether the port is in a valid range for SetPort. */
	virtual bool IsPortValid(int32 InPort) const override;

	/** Returns the port number from this address in host byte order */
	virtual int32 GetPort() const override;

	/**
	 * Sets the address structure to be bindable to any IP address.
	 * Uses the protocol the address structure was created with (or currently set to) to
	 * determine the address protocol type used.
	 *
	 * To skip assumptions, you can call the designated version explicitly below.
	 */
	virtual void SetAnyAddress() override;

	/** Explicit set to any IPv4 address */
	virtual void SetAnyIPv4Address();

	/** Explicit set to any IPv6 address */
	virtual void SetAnyIPv6Address();

	/**
	 * Sets the address structure to be bound to the multicast ip address.
	 * Uses the protocol the address structure was created with (or currently set to) to
	 * determine the address protocol type used.
	 *
	 * To skip assumptions, you can call the designated version explicitly below.
	 */
	virtual void SetBroadcastAddress() override;

	/** Explicit set to multicast IPv4 address */
	virtual void SetIPv4BroadcastAddress();

	/** Explicit set to multicast IPv6 address */
	virtual void SetIPv6BroadcastAddress();

	/**
	 * Sets the address structure to be bound to the loopback ip address.
	 * Uses the protocol the address structure was created with (or currently set to) to
	 * determine the address protocol type used.
	 *
	 * To skip assumptions, you can call the designated version explicitly below.
	 */
	virtual void SetLoopbackAddress() override;

	/** Explicit set to loopback IPv4 address */
	virtual void SetIPv4LoopbackAddress();

	/** Explicit set to loopback IPv6 address */
	virtual void SetIPv6LoopbackAddress();

	/**
	 * Converts this internet ip address to string form. String will be enclosed in square braces.
	 *
	 * @param bAppendPort whether to append the port information or not
	 */
	virtual FString ToString(bool bAppendPort) const override;

	/**
	 * Compares two internet ip addresses for equality
	 *
	 * @param Other the address to compare against
	 */
	virtual bool operator==(const FInternetAddr& Other) const override;

	/**
	 * Is this a well formed internet address, the only criteria being non-zero
	 *
	 * @return true if a valid IP, false otherwise
	 */
	virtual bool IsValid() const override;

	/**
	 * Creates a new structure with the same data as this structure
	 *
	 * @return The new structure
	 */
	virtual TSharedRef<FInternetAddr> Clone() const override;

	/**
	 * Returns the protocol family of the address data currently stored in this struct
	 *
	 * @return The type of the address. If it's not known or overridden, the address passes None.
	 */
	virtual FName GetProtocolType() const override;

	/**
	 * Returns the size of the amount of data that is being used
	 * to hold the address information. Useful for functions like bind/connect
	 *
	 * @return size of addr
	 */
	virtual SOCKLEN GetStorageSize() const;
	
	/**
	 * Sets the scope interface id of the currently held address if this address
	 * is an IPv6 address internally. If it is not, no data will be assigned.
	 * The NewScopeId must be in host byte order.
	 *
	 * @param NewScopeId the new scope interface id to set this address to
	 */
	virtual void SetScopeId(uint32 NewScopeId);

	/**
	 * Returns the IPv6 scope interface id of the currently held address
	 * if the address is an IPv6 address.
	 *
	 * @return the scope interface id
	 */
	virtual uint32 GetScopeId() const;
	
	virtual uint32 GetTypeHash() const override;

	friend class FSocketBSD;
};

#endif
