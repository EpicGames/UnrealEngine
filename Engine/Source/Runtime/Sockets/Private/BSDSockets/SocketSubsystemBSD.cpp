// Copyright Epic Games, Inc. All Rights Reserved.

#include "BSDSockets/SocketSubsystemBSD.h"

#if PLATFORM_HAS_BSD_SOCKETS || PLATFORM_HAS_BSD_IPV6_SOCKETS

#include "BSDSockets/IPAddressBSD.h"
#include "BSDSockets/SocketsBSD.h"
#include "Misc/CString.h"
#include <errno.h>

TAutoConsoleVariable<int32> CVarDisableIPv6(
	TEXT("net.DisableIPv6"),
	1,
	TEXT("If true, IPv6 will not resolve and its usage will be avoided when possible"));

FSocketBSD* FSocketSubsystemBSD::InternalBSDSocketFactory(SOCKET Socket, ESocketType SocketType, const FString& SocketDescription, const FName& SocketProtocol)
{
	// return a new socket object
	return new FSocketBSD(Socket, SocketType, SocketDescription, SocketProtocol, this);
}

FSocket* FSocketSubsystemBSD::CreateSocket(const FName& SocketType, const FString& SocketDescription, const FName& ProtocolType)
{
	SOCKET Socket = INVALID_SOCKET;
	FSocket* NewSocket = nullptr;
	int PlatformSpecificTypeFlags = 0;

	FName SocketProtocolType = ProtocolType;

	// For platforms that have two subsystems (ex: Steam) but don't explicitly inherit from SocketSubsystemBSD
	// so they don't know which protocol to end up using and pass in None.
	// This is invalid, so we need to attempt to still resolve it.
	if (ProtocolType != FNetworkProtocolTypes::IPv4 && ProtocolType != FNetworkProtocolTypes::IPv6)
	{
		SocketProtocolType = GetDefaultSocketProtocolFamily();
		// Check to make sure this is still valid.
		if (SocketProtocolType != FNetworkProtocolTypes::IPv4 && SocketProtocolType != FNetworkProtocolTypes::IPv6)
		{
			UE_LOGF(LogSockets, Warning, "Provided socket protocol type is unsupported! Returning null socket");
			return nullptr;
		}
	}

#if PLATFORM_HAS_BSD_SOCKET_FEATURE_CLOSE_ON_EXEC
	PlatformSpecificTypeFlags = SOCK_CLOEXEC;
#endif // PLATFORM_HAS_BSD_SOCKET_FEATURE_CLOSE_ON_EXEC

	bool bIsUDP = SocketType.GetComparisonIndex() == NAME_DGram;
	int32 SocketTypeFlag = (bIsUDP) ? SOCK_DGRAM : SOCK_STREAM;

	Socket = socket(GetProtocolFamilyValue(SocketProtocolType), SocketTypeFlag | PlatformSpecificTypeFlags, ((bIsUDP) ? IPPROTO_UDP : IPPROTO_TCP));

	NewSocket = (Socket != INVALID_SOCKET) ? InternalBSDSocketFactory(Socket, ((bIsUDP) ? SOCKTYPE_Datagram : SOCKTYPE_Streaming), SocketDescription, SocketProtocolType) : nullptr;

	if (!NewSocket)
	{
		UE_LOGF(LogSockets, Warning, "Failed to create socket %ls [%ls]", *SocketType.ToString(), *SocketDescription);
	}

	return NewSocket;
}

void FSocketSubsystemBSD::DestroySocket(FSocket* Socket)
{
	delete Socket;
}

FAddressInfoResult FSocketSubsystemBSD::GetAddressInfo(const TCHAR* HostName, const TCHAR* ServiceName,
	EAddressInfoFlags QueryFlags, const FName ProtocolTypeName, ESocketType SocketType)
{
	FAddressInfoResult AddrQueryResult = FAddressInfoResult(HostName, ServiceName);

	if (HostName == nullptr && ServiceName == nullptr)
	{
		UE_LOGF(LogSockets, Warning, "GetAddressInfo was passed with both a null host and service, returning empty result");
		return AddrQueryResult;
	}

#if PLATFORM_HAS_BSD_SOCKET_FEATURE_GETADDRINFO
	addrinfo* AddrInfo = nullptr;

	// Make sure we filter out IPv6 if the platform is not officially supported
	const bool bCanUseIPv6 = (CVarDisableIPv6.GetValueOnAnyThread() == 0 && PLATFORM_HAS_BSD_IPV6_SOCKETS);

	// Determine if we can save time with numericserv
	if (ServiceName != nullptr && FString(ServiceName).IsNumeric())
	{
		QueryFlags |= EAddressInfoFlags::NoResolveService;
	}

#if PLATFORM_HAS_BSD_SOCKET_FEATURE_GETNAMEINFO
	// On Windows getaddrinfo will likely return a hostname instead of FQDN in ai_canonname.
	// Try getnameinfo on every associated ip address and choose a first name with a dot if ai_canonname didn't contain FQDN.
	bool bTryGetFQDN = EnumHasAnyFlags(QueryFlags, EAddressInfoFlags::FQDomainName);
#endif

	addrinfo HintAddrInfo;
	FMemory::Memzero(&HintAddrInfo, sizeof(HintAddrInfo));
	HintAddrInfo.ai_family = GetProtocolFamilyValue(ProtocolTypeName);
	HintAddrInfo.ai_flags = GetAddressInfoHintFlag(QueryFlags);

	if (SocketType != ESocketType::SOCKTYPE_Unknown)
	{
		bool bIsUDP = (SocketType == ESocketType::SOCKTYPE_Datagram);
		HintAddrInfo.ai_protocol = bIsUDP ? IPPROTO_UDP : IPPROTO_TCP;
		HintAddrInfo.ai_socktype = bIsUDP ? SOCK_DGRAM : SOCK_STREAM;
	}

	int32 ErrorCode = getaddrinfo(TCHAR_TO_UTF8(HostName), TCHAR_TO_UTF8(ServiceName), &HintAddrInfo, &AddrInfo);
	AddrQueryResult.ReturnCode = TranslateGAIErrorCode(ErrorCode);

	UE_LOGF(LogSockets, Verbose, "Executed getaddrinfo with HostName: %ls ServiceName: %ls Return: %d", HostName ? HostName : TEXT("null"), ServiceName ? ServiceName : TEXT("null"), ErrorCode);
	if (AddrQueryResult.ReturnCode == SE_NO_ERROR)
	{
		addrinfo* AddrInfoHead = AddrInfo;
		// The canonical name will always be stored in only the first result in a getaddrinfo query
		if (AddrInfo != nullptr && AddrInfo->ai_canonname != nullptr)
		{
			AddrQueryResult.CanonicalNameResult = UTF8_TO_TCHAR(AddrInfo->ai_canonname);
			UE_LOGF(LogSockets, Verbose, "getaddrinfo ai_canonname returns: %ls", *AddrQueryResult.CanonicalNameResult);

#if PLATFORM_HAS_BSD_SOCKET_FEATURE_GETNAMEINFO
			// ai_canonname contained a dot, likely to be the best FQDN
			int32 DotPosition;
			if (AddrQueryResult.CanonicalNameResult.FindChar(TEXT('.'), DotPosition))
			{
				bTryGetFQDN = false;
			}
#endif
		}

		for (; AddrInfo != nullptr; AddrInfo = AddrInfo->ai_next)
		{
			if (AddrInfo->ai_family == AF_INET || (AddrInfo->ai_family == AF_INET6 && bCanUseIPv6))
			{
				sockaddr_storage* AddrData = reinterpret_cast<sockaddr_storage*>(AddrInfo->ai_addr);
				if (AddrData != nullptr)
				{
					TSharedRef<FInternetAddrBSD> NewAddress = MakeShareable(new FInternetAddrBSD(this));
					NewAddress->Set(*AddrData, AddrInfo->ai_addrlen);
					
					FAddressInfoResultData NewAddressData(NewAddress, AddrInfo->ai_addrlen, GetProtocolFamilyTypeName(AddrInfo->ai_family), GetSocketType(AddrInfo->ai_protocol));

					// While the FNames are the correct way to enumerate over address types, we still need to support the old format.
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					NewAddressData.AddressProtocol = GetProtocolFamilyType(AddrInfo->ai_family);
					PRAGMA_ENABLE_DEPRECATION_WARNINGS

					if (AddrQueryResult.Results.Add(NewAddressData) != INDEX_NONE)
					{
						sockaddr_in6* IPv6AddrData = reinterpret_cast<sockaddr_in6*>(AddrInfo->ai_addr);
						UE_LOGF(LogSockets, Verbose, "# Family: %ls Address: %ls And scope %d", ((AddrInfo->ai_family == AF_INET) ? TEXT("IPv4") : TEXT("IPv6")), *(NewAddress->ToString(true)), ((AddrInfo->ai_family == AF_INET) ? -1 : IPv6AddrData->sin6_scope_id));

#if PLATFORM_HAS_BSD_SOCKET_FEATURE_GETNAMEINFO
						// Try to acquire the FQDN
						if (bTryGetFQDN && (AddrData->ss_family == AF_INET || AddrData->ss_family == AF_INET6))
						{
							char HostBuffer[NI_MAXHOST];
							if (getnameinfo(
								reinterpret_cast<const sockaddr*>(AddrData), AddrInfo->ai_addrlen,
								HostBuffer, sizeof(HostBuffer),
								nullptr, 0,
								NI_NAMEREQD) == 0)
							{
								// Use HostName as FQDN if it contains a dot
								int32 DotPosition;
								if (FAnsiStringView(HostBuffer).FindChar(TEXT('.'), DotPosition))
								{
									AddrQueryResult.CanonicalNameResult = StringCast<TCHAR>(HostBuffer);
									bTryGetFQDN = false;

									UE_LOGF(LogSockets, Verbose, "Using getnameinfo hostname '%ls'", *AddrQueryResult.CanonicalNameResult);
								}
							}
						}
#endif
					}
				}
			}
		}
		freeaddrinfo(AddrInfoHead);
	}
	else
	{
		UE_LOGF(LogSockets, Warning, "GetAddressInfo failed to resolve host with error %ls [%d]", GetSocketError(AddrQueryResult.ReturnCode), ErrorCode);
	}
#else
	UE_LOGF(LogSockets, Error, "Platform has no getaddrinfo(), but did not override FSocketSubsystem::GetAddressInfo()");
#endif
	return AddrQueryResult;
}

TSharedPtr<FInternetAddr> FSocketSubsystemBSD::GetAddressFromString(const FString& InAddress)
{
	sockaddr_storage NetworkBuffer;
	FMemory::Memzero(NetworkBuffer);

	void* UniversalNetBufferPtr = nullptr;
	int32 AddrFamily = AF_INET;

#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	int32 ColonIndex;
	if (InAddress.FindChar(TEXT(':'), ColonIndex))
	{
		AddrFamily = AF_INET6;
		UniversalNetBufferPtr = &(((sockaddr_in6*)&NetworkBuffer)->sin6_addr);
	}
	else
#endif
	{
		UniversalNetBufferPtr = &(((sockaddr_in*)&NetworkBuffer)->sin_addr);
	}

	// The type of ss_family varies by platform.
	NetworkBuffer.ss_family = IntCastChecked<decltype(NetworkBuffer.ss_family)>(AddrFamily);

	if (inet_pton(AddrFamily, TCHAR_TO_ANSI(*InAddress), UniversalNetBufferPtr))
	{
		TSharedRef<FInternetAddrBSD> ReturnAddress = StaticCastSharedRef<FInternetAddrBSD>(CreateInternetAddr());
		ReturnAddress->Set(NetworkBuffer);
		return ReturnAddress;
	}
	
	ESocketErrors LastError = GetLastErrorCode();
	UE_LOGF(LogSockets, Warning, "Could not serialize %ls, got error code %ls [%d]", *InAddress, GetSocketError(LastError), LastError);
	return nullptr;
}

bool FSocketSubsystemBSD::GetMultihomeAddress(TSharedRef<FInternetAddr>& Addr)
{
	// This function is overwritten to make sure that functions that use GetLocalHostAddr/GetLocalBindAddr
	// are able to use that function to pull something like the adapter interface from all code paths.
	if (ISocketSubsystem::GetMultihomeAddress(Addr))
	{
		// Only IPv6 protocols need to handle the scope id.
		if (Addr->GetProtocolType() == FNetworkProtocolTypes::IPv6)
		{
			TArray<TSharedPtr<FInternetAddr>> AdapterAddresses;
			if (GetLocalAdapterAddresses(AdapterAddresses))
			{
				uint32 ScopeId = 0;
				for (const TSharedPtr<FInternetAddr>& AdapterAddress : AdapterAddresses)
				{
					// If the endpoint is the same, then we want to make sure that we write the scope id into it
					if (Addr->CompareEndpoints(*AdapterAddress))
					{
						ScopeId = StaticCastSharedPtr<FInternetAddrBSD>(AdapterAddress)->GetScopeId();
						break;
					}
				}

				StaticCastSharedRef<FInternetAddrBSD>(Addr)->SetScopeId(ScopeId);
			}
		}
		return true;
	}

	return false;
}

bool FSocketSubsystemBSD::GetHostName(FString& HostName)
{
#if PLATFORM_HAS_BSD_SOCKET_FEATURE_GETHOSTNAME
	ANSICHAR Buffer[256];
	bool bRead = gethostname(Buffer,256) == 0;
	if (bRead == true)
	{
		HostName = UTF8_TO_TCHAR(Buffer);
	}
	return bRead;
#else
	UE_LOGF(LogSockets, Error, "Platform has no gethostname(), but did not override FSocketSubsystem::GetHostName()");
	return false;
#endif
}

const TCHAR* FSocketSubsystemBSD::GetSocketAPIName() const
{
	return TEXT("BSD IPv4/6");
}

TSharedRef<FInternetAddr> FSocketSubsystemBSD::CreateInternetAddr()
{
	return MakeShareable(new FInternetAddrBSD(this));
}

TSharedRef<FInternetAddr> FSocketSubsystemBSD::CreateInternetAddr(const FName ProtocolType)
{
	return MakeShareable(new FInternetAddrBSD(this, ProtocolType));
}

bool FSocketSubsystemBSD::IsSocketWaitSupported() const
{
	return true;
}

ESocketErrors FSocketSubsystemBSD::GetLastErrorCode()
{
	return TranslateErrorCode(errno);
}

ESocketErrors FSocketSubsystemBSD::TranslateErrorCode(int32 Code)
{
	// @todo sockets: Windows for some reason doesn't seem to have all of the standard error messages,
	// but it overrides this function anyway - however, this 
#if !PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS

	// handle the generic -1 error
	if (Code == SOCKET_ERROR)
	{
		return GetLastErrorCode();
	}

	switch (Code)
	{
	case 0: return SE_NO_ERROR;
	case EINTR: return SE_EINTR;
	case EBADF: return SE_EBADF;
	case EACCES: return SE_EACCES;
	case EFAULT: return SE_EFAULT;
	case EINVAL: return SE_EINVAL;
	case EMFILE: return SE_EMFILE;
	case EWOULDBLOCK: return SE_EWOULDBLOCK;
	case EINPROGRESS: return SE_EINPROGRESS;
	case EALREADY: return SE_EALREADY;
	case ENOTSOCK: return SE_ENOTSOCK;
	case EDESTADDRREQ: return SE_EDESTADDRREQ;
	case EMSGSIZE: return SE_EMSGSIZE;
	case EPROTOTYPE: return SE_EPROTOTYPE;
	case ENOPROTOOPT: return SE_ENOPROTOOPT;
	case EPROTONOSUPPORT: return SE_EPROTONOSUPPORT;
	case ESOCKTNOSUPPORT: return SE_ESOCKTNOSUPPORT;
	case EOPNOTSUPP: return SE_EOPNOTSUPP;
	case EPFNOSUPPORT: return SE_EPFNOSUPPORT;
	case EAFNOSUPPORT: return SE_EAFNOSUPPORT;
	case EADDRINUSE: return SE_EADDRINUSE;
	case EADDRNOTAVAIL: return SE_EADDRNOTAVAIL;
	case ENETDOWN: return SE_ENETDOWN;
	case ENETUNREACH: return SE_ENETUNREACH;
	case ENETRESET: return SE_ENETRESET;
	case ECONNABORTED: return SE_ECONNABORTED;
	case ECONNRESET: return SE_ECONNRESET;
	case ENOBUFS: return SE_ENOBUFS;
	case EISCONN: return SE_EISCONN;
	case ENOTCONN: return SE_ENOTCONN;
	case ESHUTDOWN: return SE_ESHUTDOWN;
	case ETOOMANYREFS: return SE_ETOOMANYREFS;
	case ETIMEDOUT: return SE_ETIMEDOUT;
	case ECONNREFUSED: return SE_ECONNREFUSED;
	case ELOOP: return SE_ELOOP;
	case ENAMETOOLONG: return SE_ENAMETOOLONG;
	case EHOSTDOWN: return SE_EHOSTDOWN;
	case EHOSTUNREACH: return SE_EHOSTUNREACH;
	case ENOTEMPTY: return SE_ENOTEMPTY;
	case EUSERS: return SE_EUSERS;
	case EDQUOT: return SE_EDQUOT;
	case ESTALE: return SE_ESTALE;
	case EREMOTE: return SE_EREMOTE;
	case ENODEV: return SE_NODEV;
#if !PLATFORM_HAS_NO_EPROCLIM
	case EPROCLIM: return SE_EPROCLIM;
#endif
	case EPIPE: return SE_ECONNRESET; // for when backgrounding with an open pipe to a server
	}

#if PLATFORM_HAS_BSD_SOCKET_FEATURE_GETHOSTNAME
	switch (Code)
	{
		// some platforms may duplicate error codes, so by moving these, these error will be
		// returned, but if there are duplicate defines, it's already going to be unclear what
		// should be returned
	    case HOST_NOT_FOUND: return SE_HOST_NOT_FOUND;
	    case TRY_AGAIN: return SE_TRY_AGAIN;
	    case NO_RECOVERY: return SE_NO_RECOVERY;
    }
#endif

#endif

	UE_LOGF(LogSockets, Warning, "Unhandled socket error! Error Code: %d. Returning SE_EINVAL!", Code);
	return SE_EINVAL;
}

ESocketErrors FSocketSubsystemBSD::TranslateGAIErrorCode(int32 Code) const
{
#if PLATFORM_HAS_BSD_SOCKET_FEATURE_GETADDRINFO
	switch (Code)
	{
	// getaddrinfo() has its own error codes
	case EAI_AGAIN:			return SE_TRY_AGAIN;
	case EAI_BADFLAGS:		return SE_EINVAL;
	case EAI_FAIL:			return SE_NO_RECOVERY;
	case EAI_FAMILY:		return SE_EAFNOSUPPORT;
	case EAI_MEMORY:		return SE_ENOBUFS;
	case EAI_NONAME:		return SE_HOST_NOT_FOUND;
	case EAI_SERVICE:		return SE_EPFNOSUPPORT;
	case EAI_SOCKTYPE:		return SE_ESOCKTNOSUPPORT;
#if PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS
	case WSANO_DATA:		return SE_NO_DATA;
	case WSANOTINITIALISED: return SE_NOTINITIALISED;
#else			
	case EAI_NODATA:		return SE_NO_DATA;
	case EAI_ADDRFAMILY:	return SE_ADDRFAMILY;
	case EAI_SYSTEM:		return SE_SYSTEM;
#endif
	case 0:					break; // 0 means success
	default:
		UE_LOGF(LogSockets, Warning, "Unhandled getaddrinfo() socket error! Code: %d", Code);
		return SE_EINVAL;
	}
#endif // PLATFORM_HAS_BSD_SOCKET_FEATURE_GETADDRINFO

	return SE_NO_ERROR;
}

int32 FSocketSubsystemBSD::GetProtocolFamilyValue(const FName& InProtocol) const
{
	if (InProtocol == FNetworkProtocolTypes::IPv4)
	{
		return AF_INET;
	}
	else if (InProtocol == FNetworkProtocolTypes::IPv6)
	{
		return AF_INET6;
	}

	return AF_UNSPEC;
}

const FName FSocketSubsystemBSD::GetProtocolFamilyTypeName(int32 InProtocol) const
{
	switch (InProtocol)
	{
		default:
		case AF_UNSPEC: return NAME_None;
		case AF_INET:   return FNetworkProtocolTypes::IPv4;
		case AF_INET6:  return FNetworkProtocolTypes::IPv6;
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
int32 FSocketSubsystemBSD::GetProtocolFamilyValue(ESocketProtocolFamily InProtocol) const
{
	switch (InProtocol)
	{
		default:
		case ESocketProtocolFamily::None: return AF_UNSPEC;
		case ESocketProtocolFamily::IPv4: return AF_INET;
		case ESocketProtocolFamily::IPv6: return AF_INET6;
	}
}

ESocketProtocolFamily FSocketSubsystemBSD::GetProtocolFamilyType(int32 InProtocol) const
{
	switch (InProtocol)
	{
		default:
		case AF_UNSPEC: return ESocketProtocolFamily::None;
		case AF_INET:   return ESocketProtocolFamily::IPv4;
		case AF_INET6:  return ESocketProtocolFamily::IPv6;
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

ESocketType FSocketSubsystemBSD::GetSocketType(int32 InSocketType) const
{
	switch (InSocketType)
	{
		case SOCK_STREAM:
		case IPPROTO_TCP: return ESocketType::SOCKTYPE_Streaming;
		case SOCK_DGRAM:
		case IPPROTO_UDP: return ESocketType::SOCKTYPE_Datagram;
		default: return SOCKTYPE_Unknown;
	}
}

int32 FSocketSubsystemBSD::GetAddressInfoHintFlag(EAddressInfoFlags InFlags) const
{
	int32 ReturnFlagsCode = 0;

#if PLATFORM_HAS_BSD_SOCKET_FEATURE_GETADDRINFO
	if (InFlags == EAddressInfoFlags::Default)
	{
		return ReturnFlagsCode;
	}

	if (EnumHasAnyFlags(InFlags, EAddressInfoFlags::NoResolveHost))
	{
		ReturnFlagsCode |= AI_NUMERICHOST;
	}

	if (EnumHasAnyFlags(InFlags, EAddressInfoFlags::NoResolveService))
	{
		ReturnFlagsCode |= AI_NUMERICSERV;
	}

	if (EnumHasAnyFlags(InFlags, EAddressInfoFlags::OnlyUsableAddresses))
	{
		ReturnFlagsCode |= AI_ADDRCONFIG;
	}

	if (EnumHasAnyFlags(InFlags, EAddressInfoFlags::BindableAddress))
	{
		ReturnFlagsCode |= AI_PASSIVE;
	}

	/* This means nothing unless AI_ALL is also specified. */
	if (EnumHasAnyFlags(InFlags, EAddressInfoFlags::AllowV4MappedAddresses))
	{
		ReturnFlagsCode |= AI_V4MAPPED;
	}

	if (EnumHasAnyFlags(InFlags, EAddressInfoFlags::AllResults))
	{
		ReturnFlagsCode |= AI_ALL;
	}

	if (EnumHasAnyFlags(InFlags, EAddressInfoFlags::CanonicalName))
	{
		ReturnFlagsCode |= AI_CANONNAME;
	}

	if (EnumHasAnyFlags(InFlags, EAddressInfoFlags::FQDomainName))
	{
#ifdef AI_FQDN
		ReturnFlagsCode |= AI_FQDN;
#else
		ReturnFlagsCode |= AI_CANONNAME;
#endif
	}
#endif

	return ReturnFlagsCode;
}

TSharedRef<FInternetAddr> FSocketSubsystemBSD::GetLocalHostAddr(FOutputDevice& Out, bool& bCanBindAll)
{
	TSharedRef<FInternetAddr> HostAddr = CreateInternetAddr();
	bCanBindAll = false;

	if (!GetMultihomeAddress(HostAddr))
	{
		// First try to get the host address via connect
		if (!GetLocalHostAddrViaConnect(HostAddr))
		{
			bCanBindAll = true;

			TArray<TSharedPtr<FInternetAddr>> AdapterAddresses;
			if (!GetLocalAdapterAddresses(AdapterAddresses) || (AdapterAddresses.Num() == 0))
			{
				Out.Logf(TEXT("Could not fetch the local adapter addresses"));
				HostAddr->SetAnyAddress();
			}
			else
			{
				if (AdapterAddresses.Num() > 0)
				{
					HostAddr = AdapterAddresses[0]->Clone();
				}
			}
		}
	}

	UE_LOGF(LogSockets, VeryVerbose, "GetLocalHostAddr: %ls", *HostAddr->ToString(true));

	return HostAddr;
}

bool FSocketSubsystemBSD::GetLocalHostAddrViaConnect(TSharedRef<FInternetAddr>& HostAddr)
{
	bool bReturnValue = false;

	const bool bDisableIPv6 = CVarDisableIPv6.GetValueOnAnyThread() == 1;

	TSharedRef<FInternetAddr> ConnectAddr = bDisableIPv6 ? CreateInternetAddr(FNetworkProtocolTypes::IPv4) : CreateInternetAddr();

	bool bIsValid = false;
	// any IP will do, doesn't even need to be reachable
	if (ConnectAddr->GetProtocolType() == FNetworkProtocolTypes::IPv6)
	{
		ConnectAddr->SetIp(TEXT("::ffff:172.31.255.255"), bIsValid);		
	}
	else if (ConnectAddr->GetProtocolType() == FNetworkProtocolTypes::IPv4)
	{
		ConnectAddr->SetIp(TEXT("172.31.255.255"), bIsValid);
	}

	if (bIsValid)
	{
		ConnectAddr->SetPort(256);

		FUniqueSocket TestSocket = CreateUniqueSocket(NAME_DGram, TEXT("GetLocalHostAddrViaConnect"), ConnectAddr->GetProtocolType());

		if (TestSocket.IsValid())
		{
			if (TestSocket->Connect(*ConnectAddr))
			{
				TestSocket->GetAddress(*HostAddr);

				HostAddr->SetPort(0);
				bReturnValue = true;

				UE_LOGF(LogSockets, VeryVerbose, "GetLocalHostAddrViaConnect: %ls", *HostAddr->ToString(true));
			}

			TestSocket->Close();
		}
	}

	return bReturnValue;
}

void FSocketSubsystemBSD::QuerySocketStateMany(TArrayView<FSocket*> Sockets, TArray<ESocketStateFlags>& InOutSocketStates, FTimespan WaitTime) const
{
	check(IsSocketWaitSupported());
	check(InOutSocketStates.Num() == Sockets.Num());

	if (Sockets.Num() == 0)
	{
		return;
	}

#if PLATFORM_HAS_BSD_SOCKET_FEATURE_POLL && USE_SOCKET_FEATURE_POLL

	TArray<struct pollfd> FDSet;
	FDSet.SetNum(Sockets.Num());

	for (int32 SocketIndex = 0; SocketIndex < Sockets.Num(); SocketIndex++)
	{
		uint8 Request = (uint8)InOutSocketStates[SocketIndex];
		FSocket* Socket = Sockets[SocketIndex];

		struct pollfd FD;
		FD.fd = static_cast<FSocketBSD*>(Socket)->GetNativeSocket();
		FD.events = 0;
		FD.revents = 0;

		if (Request & (uint8)ESocketStateFlags::Readable)
		{
			FD.events |= POLLIN;
		}

		if (Request & (uint8)ESocketStateFlags::Writable)
		{
			FD.events |= POLLOUT;
		}

		// POLLERR is output-only and always reported in revents; no need to set it in events.

		FDSet[SocketIndex] = FD;
	}

	int PollResult = ::poll(&FDSet[0], FDSet.Num(), WaitTime.GetTicks() >= 0 ? (int)WaitTime.GetTotalMilliseconds() : -1);

	if (PollResult >= 0)
	{
		for (int32 SocketIndex = 0; SocketIndex < Sockets.Num(); SocketIndex++)
		{
			uint8 Request = (uint8)InOutSocketStates[SocketIndex];
			uint8 Result = 0;

			struct pollfd FD = FDSet[SocketIndex];

			if ((Request & (uint8)ESocketStateFlags::Readable) && (FD.revents & (POLLIN | POLLHUP)))
			{
				Result |= (uint8)ESocketStateFlags::Readable;
			}

			if ((Request & (uint8)ESocketStateFlags::Writable) && (FD.revents & POLLOUT))
			{
				Result |= (uint8)ESocketStateFlags::Writable;
			}

			if ((Request & (uint8)ESocketStateFlags::Error) && (FD.revents & (POLLERR | POLLNVAL)))
			{
				Result |= (uint8)ESocketStateFlags::Error;
			}

			InOutSocketStates[SocketIndex] = (ESocketStateFlags)Result;
		}
	}
	else
	{
		for (int32 SocketIndex = 0; SocketIndex < InOutSocketStates.Num(); SocketIndex++)
		{
			InOutSocketStates[SocketIndex] = (ESocketStateFlags)0;
		}
	}

#elif PLATFORM_HAS_BSD_SOCKET_FEATURE_SELECT

	check(Sockets.Num() <= FD_SETSIZE);	

	// convert WaitTime to a timeval
	timeval Time;
	Time.tv_sec = (int32)WaitTime.GetTotalSeconds();
	Time.tv_usec = WaitTime.GetFractionMicro();

	fd_set SocketSetRead;
	fd_set SocketSetWrite;
	fd_set SocketSetError;

	FD_ZERO(&SocketSetRead);
	FD_ZERO(&SocketSetWrite);
	FD_ZERO(&SocketSetError);

	int MaxFd = 0;
	for (int32 SocketIndex = 0; SocketIndex < Sockets.Num(); SocketIndex++)
	{
		uint8 Request = (uint8)InOutSocketStates[SocketIndex];

		FSocket* Socket = Sockets[SocketIndex];
		SOCKET PlatformSocket = static_cast<FSocketBSD*>(Socket)->GetNativeSocket();

		if (Request & (uint8)ESocketStateFlags::Readable)
		{
			FD_SET(PlatformSocket, &SocketSetRead);
		}

		if (Request & (uint8)ESocketStateFlags::Writable)
		{
			FD_SET(PlatformSocket, &SocketSetWrite);
		}

		if (Request & (uint8)ESocketStateFlags::Error)
		{
			FD_SET(PlatformSocket, &SocketSetError);
		}

		MaxFd = FMath::Max(MaxFd, (int)PlatformSocket);
	}

	timeval* TimePointer = WaitTime.GetTicks() >= 0 ? &Time : nullptr;

	// Check the status of the state
	int32 SelectStatus = select(MaxFd+1, &SocketSetRead, &SocketSetWrite, &SocketSetError, TimePointer);

	if (SelectStatus >= 0)
	{
		for (int32 SocketIndex = 0; SocketIndex < Sockets.Num(); SocketIndex++)
		{
			uint8 Result = 0;

			FSocket* Socket = Sockets[SocketIndex];
			SOCKET PlatformSocket = static_cast<FSocketBSD*>(Socket)->GetNativeSocket();

			if (FD_ISSET(PlatformSocket, &SocketSetRead))
			{
				Result |= (uint8)ESocketStateFlags::Readable;
			}

			if (FD_ISSET(PlatformSocket, &SocketSetWrite))
			{
				Result |= (uint8)ESocketStateFlags::Writable;
			}

			if (FD_ISSET(PlatformSocket, &SocketSetError))
			{
				Result |= (uint8)ESocketStateFlags::Error;
			}

			InOutSocketStates[SocketIndex] = (ESocketStateFlags)Result;
		}
	}
	else
	{
		for (int32 SocketIndex = 0; SocketIndex < InOutSocketStates.Num(); SocketIndex++)
		{
			InOutSocketStates[SocketIndex] = (ESocketStateFlags)0;
		}
	}
#else
	UE_LOGF(LogSockets, Fatal, "This platform doesn't support poll() or select(), but FSocketSubsystemBSD::QuerySocketStateMany was not implemented");
#endif
}

#endif	//PLATFORM_HAS_BSD_SOCKETS
