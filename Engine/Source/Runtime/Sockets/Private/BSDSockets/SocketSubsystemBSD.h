// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/OutputDeviceRedirector.h"
#include "HAL/IConsoleManager.h"
#include "SocketSubsystemPackage.h"
#include "SocketSubsystem.h"
#include "IPAddress.h"

#if PLATFORM_HAS_BSD_SOCKETS || PLATFORM_HAS_BSD_IPV6_SOCKETS

class FSocketBSD;
extern TAutoConsoleVariable<int32> CVarDisableIPv6;

#include "SocketSubsystemBSDPrivate.h"

/**
 * Standard BSD specific socket subsystem implementation
 */
class FSocketSubsystemBSD : public ISocketSubsystem
{
public:
	/**
	 * Specifies the default socket protocol family to use when creating a socket
	 * without explicitly passing in the protocol type on creation.
	 *
	 * This function is mostly here for backwards compatibility. For best practice, moving to
	 * the new CreateSocket that takes a protocol is advised.
	 *
	 * All sockets created using the base class's CreateSocket will use this function
	 * to determine domain.
	 */
	virtual FName GetDefaultSocketProtocolFamily() const
	{
		return FNetworkProtocolTypes::IPv4;
	}


	// ISocketSubsystem interface
	virtual TSharedRef<FInternetAddr> CreateInternetAddr() override;
	virtual TSharedRef<FInternetAddr> CreateInternetAddr(const FName ProtocolType) override;

	virtual class FSocket* CreateSocket(const FName& SocketType, const FString& SocketDescription, bool bForceUDP = false) override
	{
		return CreateSocket(SocketType, SocketDescription, GetDefaultSocketProtocolFamily());
	}

	virtual class FSocket* CreateSocket(const FName& SocketType, const FString& SocketDescription, const FName& ProtocolType) override;

	virtual void DestroySocket( class FSocket* Socket ) override;

	virtual FAddressInfoResult GetAddressInfo(const TCHAR* HostName, const TCHAR* ServiceName = nullptr,
		EAddressInfoFlags QueryFlags = EAddressInfoFlags::Default,
		const FName ProtocolTypeName = NAME_None,
		ESocketType SocketType = ESocketType::SOCKTYPE_Unknown) override;

	virtual TSharedPtr<FInternetAddr> GetAddressFromString(const FString& InAddress) override;

	virtual bool GetMultihomeAddress(TSharedRef<FInternetAddr>& Addr) override;

	virtual bool GetHostName(FString& HostName) override;
	virtual ESocketErrors GetLastErrorCode() override;

	virtual const TCHAR* GetSocketAPIName() const override;

	virtual bool RequiresChatDataBeSeparate() override
	{
		return false;
	}

	virtual bool RequiresEncryptedPackets() override
	{
		return false;
	}

	virtual ESocketErrors TranslateErrorCode( int32 Code ) override;

	virtual bool IsSocketWaitSupported() const override;

	virtual TSharedRef<FInternetAddr> GetLocalHostAddr(FOutputDevice& Out, bool& bCanBindAll) override;

	/**
	 * Translates an ESocketProtocolFamily code into a value usable by raw socket apis.
	 */
	UE_DEPRECATED(4.23, "Switch to the FName version for scalable protocol support")
	virtual int32 GetProtocolFamilyValue(ESocketProtocolFamily InProtocol) const;
	virtual int32 GetProtocolFamilyValue(const FName& InProtocol) const;
	
	/**
	 * Translates an raw socket family type value into a protocol name that can be used by the network layer.
	 */
	virtual const FName GetProtocolFamilyTypeName(int32 InProtocol) const;

	UE_DEPRECATED(4.23, "Use GetProtocolFamilyTypeName")
	virtual ESocketProtocolFamily GetProtocolFamilyType(int32 InProtocol) const;
	

	/**
	 * Translates an raw socket protocol type value into an enum that can be used by the network layer.
	 */
	virtual ESocketType GetSocketType(int32 InSocketType) const;

	/**
	 * Translates an ESocketAddressInfoFlags into a value usable by getaddrinfo
	 */
	virtual int32 GetAddressInfoHintFlag(EAddressInfoFlags InFlags) const;

	/** Polls multiple sockets for readability, writability, and error states using poll() or select().
	 *  InOutSocketStates is used as both input (requested flags) and output (resulting flags).
	 */
	void QuerySocketStateMany(TArrayView<FSocket*> Sockets, TArray<ESocketStateFlags>& InOutSocketStates, FTimespan WaitTime) const override;

protected:
	/**
	 * Translates return values of getaddrinfo() to socket error enum
	 */
	ESocketErrors TranslateGAIErrorCode(int32 Code) const;

	/**
	 * Allows a subsystem subclass to create a FSocketBSD sub class.
	 */
	virtual class FSocketBSD* InternalBSDSocketFactory( SOCKET Socket, ESocketType SocketType, const FString& SocketDescription, const FName& SocketProtocol);

	/**
	 * Allows a subsystem subclass to create a FSocketBSD sub class.
	 */
	UE_DEPRECATED(4.22, "To support multiple stack types, move to the constructor that allows for specifying the protocol stack to initialize the socket on.")
	virtual class FSocketBSD* InternalBSDSocketFactory(SOCKET Socket, ESocketType SocketType, const FString& SocketDescription)
	{
		return InternalBSDSocketFactory(Socket, SocketType, SocketDescription, GetDefaultSocketProtocolFamily());
	}

	/**
	 * Helper function that attempts to determine the host address via socket connect
	 *
	 * @param HostAddr The address if successful
	 * @return true if successful, false otherwise
	 */
	bool GetLocalHostAddrViaConnect(TSharedRef<FInternetAddr>& HostAddr);

	// allow BSD sockets to use this when creating new sockets from accept() etc
	friend FSocketBSD;
};

#endif
