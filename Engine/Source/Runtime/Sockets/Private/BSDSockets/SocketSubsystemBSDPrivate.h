// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SocketSubsystem.h"

#if PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS
	#include "Windows/AllowWindowsPlatformTypes.h"

	#include <winsock2.h>
	#include <ws2tcpip.h>

	typedef int32 SOCKLEN;

	#include "Windows/HideWindowsPlatformTypes.h"
#else
#ifdef UE_SOCKET_API_WRAPPER
	#include UE_SOCKET_API_WRAPPER
#else
	#include <unistd.h>
	#include <sys/socket.h>
#if PLATFORM_HAS_BSD_SOCKET_FEATURE_IOCTL
	#include <fcntl.h>
	#include <sys/types.h>
	#include <sys/ioctl.h>
#endif
#if PLATFORM_HAS_BSD_SOCKET_FEATURE_POLL
	#include <poll.h>
#endif
#ifndef USE_SOCKET_FEATURE_POLL
	#define USE_SOCKET_FEATURE_POLL 1
#endif
	#include <netinet/in.h>
	#include <arpa/inet.h>
#if PLATFORM_HAS_BSD_SOCKET_FEATURE_GETHOSTNAME
	#include <netdb.h>
#endif
#if PLATFORM_HAS_BSD_SOCKET_FEATURE_NODELAY
	#include <netinet/tcp.h>
#endif

	#define ioctlsocket ioctl
#endif

	#define SOCKET_ERROR -1
	#define INVALID_SOCKET -1

	typedef socklen_t SOCKLEN;
	typedef int32 SOCKET;
	typedef sockaddr_in SOCKADDR_IN;
	typedef struct timeval TIMEVAL;

	inline int32 closesocket(SOCKET Socket)
	{
		shutdown(Socket, SHUT_RDWR); // gracefully shutdown if connected
		return close(Socket);
	}

#endif

// Since the flag constants may have different values per-platform, translate into corresponding system constants.
// For example, MSG_WAITALL is 0x8 on Windows, but 0x100 on other platforms.
inline int TranslateFlags(ESocketReceiveFlags::Type Flags)
{
	int TranslatedFlags = 0;

	if (Flags & ESocketReceiveFlags::Peek)
	{
		TranslatedFlags |= MSG_PEEK;
#if PLATFORM_HAS_BSD_SOCKET_FEATURE_MSG_DONTWAIT
		TranslatedFlags |= MSG_DONTWAIT;
#endif // PLATFORM_HAS_BSD_SOCKET_FEATURE_MSG_DONTWAIT
	}

	if (Flags & ESocketReceiveFlags::WaitAll)
	{
		TranslatedFlags |= MSG_WAITALL;
	}

	return TranslatedFlags;
}
