// Copyright Epic Games, Inc. All Rights Reserved.

#include "BSDSockets/SocketsBSD.h"

#if PLATFORM_HAS_BSD_SOCKETS || PLATFORM_HAS_BSD_IPV6_SOCKETS

#include "BSDSockets/IPAddressBSD.h"
#include "BSDSockets/SocketSubsystemBSD.h"
//#include "Net/NetworkProfiler.h"

/* FSocket overrides
 *****************************************************************************/

bool FSocketBSD::Shutdown(ESocketShutdownMode Mode)
{
	int InternalMode = 0;
	// Windows uses different constants than POSIX
#if !PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS
	switch (Mode)
	{
		case ESocketShutdownMode::Read:
			InternalMode = SHUT_RD;
			break;
		case ESocketShutdownMode::Write:
			InternalMode = SHUT_WR;
			break;
		case ESocketShutdownMode::ReadWrite:
			InternalMode = SHUT_RDWR;
			break;
	}
#endif
	return shutdown(Socket, InternalMode) == 0;
}

UPTRINT FSocketBSD::ReleaseNativeSocket()
{
	UPTRINT NativeSocket = Socket;
	Socket = INVALID_SOCKET;

	return NativeSocket;
}

bool FSocketBSD::Close(void)
{
	if (Socket != INVALID_SOCKET)
	{
		int32 error = closesocket(Socket);
		Socket = INVALID_SOCKET;
		return error == 0;
	}
	return false;
}

bool FSocketBSD::Bind(const FInternetAddr& Addr)
{
	if (Addr.GetProtocolType() != GetProtocol())
	{
		UE_LOGF(LogSockets, Warning, "Tried to bind address with protocol %ls to a socket with protocol %ls",
			*Addr.GetProtocolType().ToString(), *GetProtocol().ToString());
		return false;
	}

	const FInternetAddrBSD& BSDAddr = static_cast<const FInternetAddrBSD&>(Addr);
	return bind(Socket, (const sockaddr*)&(BSDAddr.Addr), BSDAddr.GetStorageSize()) == 0;
}

bool FSocketBSD::Connect(const FInternetAddr& Addr)
{
	if (Addr.GetProtocolType() != GetProtocol())
	{
		UE_LOGF(LogSockets, Warning, "Tried to connect with an address with protocol %ls using a socket with protocol %ls", 
			*Addr.GetProtocolType().ToString(), *GetProtocol().ToString());
		return false;
	}

	const FInternetAddrBSD& BSDAddr = static_cast<const FInternetAddrBSD&>(Addr);
	int32 Return = connect(Socket, (const sockaddr*)&(BSDAddr.Addr), BSDAddr.GetStorageSize());
	
	check(SocketSubsystem);
	ESocketErrors Error = SocketSubsystem->TranslateErrorCode(Return);

	// EWOULDBLOCK is not an error, and EINPROGRESS is fine on initial connection as it may still be creating for nonblocking sockets
	return ((Error == SE_NO_ERROR) || (Error == SE_EWOULDBLOCK) || (Error == SE_EINPROGRESS));
}


bool FSocketBSD::Listen(int32 MaxBacklog)
{
	return listen(Socket, MaxBacklog) == 0;
}

bool FSocketBSD::WaitForPendingConnection(bool& bHasPendingConnection, const FTimespan& WaitTime)
{
	bool bHasSucceeded = false;
	bHasPendingConnection = false;

	// make sure socket has no error state
	if (HasState(ESocketBSDParam::HasError) == ESocketBSDReturn::No)
	{
		// get the read state
		ESocketBSDReturn State = HasState(ESocketBSDParam::CanRead, WaitTime);
		
		// turn the result into the outputs
		bHasSucceeded = State != ESocketBSDReturn::EncounteredError;
		bHasPendingConnection = State == ESocketBSDReturn::Yes;
	}

	return bHasSucceeded;
}

bool FSocketBSD::HasPendingData(uint32& PendingDataSize)
{
	PendingDataSize = 0;

	// make sure socket has no error state
	if (HasState(ESocketBSDParam::CanRead) == ESocketBSDReturn::Yes)
	{
#if PLATFORM_HAS_BSD_SOCKET_FEATURE_IOCTL
		// See if there is any pending data on the read socket
		if (ioctlsocket(Socket, FIONREAD, (u_long*)(&PendingDataSize)) == 0)
#endif
		{
			return (PendingDataSize > 0);
		}
	}

	return false;
}

FSocket* FSocketBSD::Accept(const FString& InSocketDescription)
{
	SOCKET NewSocket = accept(Socket, NULL, NULL);

	if (NewSocket != INVALID_SOCKET)
	{
		// we need the subclass to create the actual FSocket object
		check(SocketSubsystem);
		FSocketSubsystemBSD* BSDSystem = static_cast<FSocketSubsystemBSD*>(SocketSubsystem);
		return BSDSystem->InternalBSDSocketFactory(NewSocket, SocketType, InSocketDescription, SocketProtocol);
	}

	return NULL;
}

FSocket* FSocketBSD::Accept(FInternetAddr& OutAddr, const FString& InSocketDescription)
{
	FInternetAddrBSD& BSDAddr = static_cast<FInternetAddrBSD&>(OutAddr);
	SOCKLEN SizeOf = sizeof(sockaddr_storage);
	SOCKET NewSocket = accept(Socket, (sockaddr*)&(BSDAddr.Addr), &SizeOf);

	if (NewSocket != INVALID_SOCKET)
	{
		// we need the subclass to create the actual FSocket object
		check(SocketSubsystem);
		FSocketSubsystemBSD* BSDSystem = static_cast<FSocketSubsystemBSD*>(SocketSubsystem);
		return BSDSystem->InternalBSDSocketFactory(NewSocket, SocketType, InSocketDescription, BSDAddr.GetProtocolType());
	}

	return NULL;
}

bool FSocketBSD::SendTo(const uint8* Data, int32 Count, int32& BytesSent, const FInternetAddr& Destination)
{
	// TODO: Consider converting IPv4 addresses to v6 when needed
	if (Destination.GetProtocolType() != GetProtocol())
	{
		UE_LOGF(LogSockets, Warning, "Destination protocol of '%ls' does not match protocol: '%ls' for address: '%ls'",
			*Destination.GetProtocolType().ToString(), *GetProtocol().ToString(), *Destination.ToString(true));
		return false;
	}

	const FInternetAddrBSD& BSDAddr = static_cast<const FInternetAddrBSD&>(Destination);
	// Write the data and see how much was written
	BytesSent = sendto(Socket, (const char*)Data, Count, SendFlags, (const sockaddr*)&(BSDAddr.Addr), BSDAddr.GetStorageSize());

//	NETWORK_PROFILER(FSocket::SendTo(Data,Count,BytesSent,Destination));

	bool Result = BytesSent >= 0;
	if (Result)
	{
		LastActivityTime = FPlatformTime::Seconds();
	}
	return Result;
}

bool FSocketBSD::Send(const uint8* Data, int32 Count, int32& BytesSent)
{
	BytesSent = send(Socket,(const char*)Data,Count,SendFlags);

//	NETWORK_PROFILER(FSocket::Send(Data,Count,BytesSent));

	bool Result = BytesSent >= 0;
	if (Result)
	{
		LastActivityTime = FPlatformTime::Seconds();
	}
	return Result;
}

bool FSocketBSD::RecvFrom(uint8* Data, int32 BufferSize, int32& BytesRead, FInternetAddr& Source, ESocketReceiveFlags::Type Flags)
{
	bool bSuccess = false;
	const bool bStreamSocket = (SocketType == SOCKTYPE_Streaming);
	const int TranslatedFlags = TranslateFlags(Flags);
	FInternetAddrBSD& BSDAddr = static_cast<FInternetAddrBSD&>(Source);
	SOCKLEN Size = sizeof(sockaddr_storage);
	sockaddr* Addr = (sockaddr*)&(BSDAddr.Addr);

	// Read into the buffer and set the source address
	BytesRead = recvfrom(Socket, (char*)Data, BufferSize, TranslatedFlags, Addr, &Size);
//	NETWORK_PROFILER(FSocket::RecvFrom(Data,BufferSize,BytesRead,Source));

	if (BytesRead >= 0)
	{
		// For Streaming sockets, 0 indicates a graceful failure
		bSuccess = !bStreamSocket || (BytesRead > 0);
	}
	else
	{
		// For Streaming sockets, don't treat SE_EWOULDBLOCK as an error
		bSuccess = bStreamSocket && (SocketSubsystem->TranslateErrorCode(BytesRead) == SE_EWOULDBLOCK);
		BytesRead = 0;
	}

	if (bSuccess)
	{
		LastActivityTime = FPlatformTime::Seconds();
	}

	return bSuccess;
}

bool FSocketBSD::RecvFromWithPktInfo(uint8* Data, int32 BufferSize, int32& BytesRead, FInternetAddr& Source, FInternetAddr& Destination, ESocketReceiveFlags::Type Flags)
{
#if defined(IP_PKTINFO) && (PLATFORM_LINUX || PLATFORM_MAC)
#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	if (GetProtocol() == FNetworkProtocolTypes::IPv6)
	{
		// Invalid combination
		const FInternetAddrBSD& BSDAddr = static_cast<const FInternetAddrBSD&>(Destination);
		if (BSDAddr.GetProtocolType() != FNetworkProtocolTypes::IPv6)
		{
			return false;
		}
	}
#endif // PLATFORM_HAS_BSD_IPV6_SOCKETS
	bool bSuccess = false;
	const bool bStreamSocket = (SocketType == SOCKTYPE_Streaming);
	const int TranslatedFlags = TranslateFlags(Flags);
	FInternetAddrBSD& BSDAddr = static_cast<FInternetAddrBSD&>(Source);
	const SOCKLEN Size = sizeof(sockaddr_storage);
	sockaddr* Addr = (sockaddr*)BSDAddr.GetRawAddr();
	constexpr int32 ControlMsgSize = 1024;
	char ControlMsg[ControlMsgSize];
	iovec IOBuf;
	IOBuf.iov_base = Data;
	IOBuf.iov_len = static_cast<size_t>(BufferSize);
	msghdr Msg = {};
	Msg.msg_name = Addr;
	Msg.msg_namelen = Size;
	Msg.msg_iov = &IOBuf;
	Msg.msg_iovlen = 1;
	Msg.msg_control = ControlMsg;
	Msg.msg_controllen = ControlMsgSize;
	const ssize_t Result = recvmsg(Socket, &Msg, TranslatedFlags);
	if (Result >= 0)
	{
		BytesRead = static_cast<int32>(Result);
		// iterate each message searching for IP_PKTINFO
		for (cmsghdr* CMsg = CMSG_FIRSTHDR(&Msg); CMsg != nullptr; CMsg = CMSG_NXTHDR(&Msg, CMsg))
		{
			if (CMsg->cmsg_type != IP_PKTINFO)
			{
				continue;
			}
#if PLATFORM_HAS_BSD_IPV6_SOCKETS
			if (GetProtocol() == FNetworkProtocolTypes::IPv6)
			{
				in6_pktinfo* PktInfo = (in6_pktinfo*)CMSG_DATA(CMsg);
				uint8* DestinationAddr = (uint8*)&PktInfo->ipi6_addr;
				constexpr int32 DestinationAddrSize = 16;
				TArray<uint8> DestinationAddrRaw;
				DestinationAddrRaw.AddUninitialized(DestinationAddrSize);
				for (int32 ByteIndex = 0; ByteIndex < DestinationAddrSize; ++ByteIndex)
				{
					DestinationAddrRaw[ByteIndex] = DestinationAddr[ByteIndex];
				}
				Destination.SetRawIp(DestinationAddrRaw);
			}
			else
			{
#endif // PLATFORM_HAS_BSD_IPV6_SOCKETS
				in_pktinfo* PktInfo = (in_pktinfo*)CMSG_DATA(CMsg);
				uint8* DestinationAddr = (uint8*)&PktInfo->ipi_addr;
				TArray<uint8> DestinationAddrRaw = { DestinationAddr[0], DestinationAddr[1], DestinationAddr[2], DestinationAddr[3] };
				Destination.SetRawIp(DestinationAddrRaw);
#if PLATFORM_HAS_BSD_IPV6_SOCKETS
			}
#endif // PLATFORM_HAS_BSD_IPV6_SOCKETS
			break;
		}
		// For Streaming sockets, 0 indicates a graceful failure
		bSuccess = !bStreamSocket || (BytesRead > 0);
	}
	else
	{
		// For Streaming sockets, don't treat SE_EWOULDBLOCK as an error as we will potentially retry later
		bSuccess = bStreamSocket && (SocketSubsystem->TranslateErrorCode(Result) == SE_EWOULDBLOCK);
		BytesRead = 0;
	}
	if (bSuccess)
	{
		LastActivityTime = FPlatformTime::Seconds();
	}
	return bSuccess;
#else
	return false;
#endif // defined(IP_PKTINFO) && (PLATFORM_LINUX || PLATFORM_MAC)
}

bool FSocketBSD::Recv(uint8* Data, int32 BufferSize, int32& BytesRead, ESocketReceiveFlags::Type Flags)
{
	bool bSuccess = false;
	const bool bStreamSocket = (SocketType == SOCKTYPE_Streaming);
	const int TranslatedFlags = TranslateFlags(Flags);
	BytesRead = recv(Socket, (char*)Data, BufferSize, TranslatedFlags);

//	NETWORK_PROFILER(FSocket::Recv(Data,BufferSize,BytesRead));

	if (BytesRead >= 0)
	{
		// For Streaming sockets, 0 indicates a graceful failure
		bSuccess = !bStreamSocket || (BytesRead > 0);
	}
	else
	{
		// For Streaming sockets, don't treat SE_EWOULDBLOCK as an error
		bSuccess = bStreamSocket && (SocketSubsystem->TranslateErrorCode(BytesRead) == SE_EWOULDBLOCK);
		BytesRead = 0;
	}

	if (bSuccess)
	{
		LastActivityTime = FPlatformTime::Seconds();
	}

	return bSuccess;
}

bool FSocketBSD::Wait(ESocketWaitConditions::Type Condition, FTimespan WaitTime)
{
	if ((Condition == ESocketWaitConditions::WaitForRead) || (Condition == ESocketWaitConditions::WaitForReadOrWrite))
	{
		if (HasState(ESocketBSDParam::CanRead, WaitTime) == ESocketBSDReturn::Yes)
		{
			return true;
		}
	}

	if ((Condition == ESocketWaitConditions::WaitForWrite) || (Condition == ESocketWaitConditions::WaitForReadOrWrite))
	{
		if (HasState(ESocketBSDParam::CanWrite, WaitTime) == ESocketBSDReturn::Yes)
		{
			return true;
		}
	}

	return false;
}

ESocketConnectionState FSocketBSD::GetConnectionState(void)
{
	ESocketConnectionState CurrentState = SCS_ConnectionError;

	// look for an existing error
	if (Socket != INVALID_SOCKET && HasState(ESocketBSDParam::HasError) == ESocketBSDReturn::No)
	{
		if (FPlatformTime::Seconds() - LastActivityTime > 5.0)
		{
			// get the write state
			ESocketBSDReturn WriteState = HasState(ESocketBSDParam::CanWrite, FTimespan::FromMilliseconds(1));
			ESocketBSDReturn ReadState = HasState(ESocketBSDParam::CanRead, FTimespan::FromMilliseconds(1));
		
			// translate yes or no (error is already set)
			if (WriteState == ESocketBSDReturn::Yes || ReadState == ESocketBSDReturn::Yes)
			{
				CurrentState = SCS_Connected;
				LastActivityTime = FPlatformTime::Seconds();
			}
			else if (WriteState == ESocketBSDReturn::No && ReadState == ESocketBSDReturn::No)
			{
				CurrentState = SCS_NotConnected;
			}
		}
		else
		{
			CurrentState = SCS_Connected;
		}
	}

	return CurrentState;
}

void FSocketBSD::GetAddress(FInternetAddr& OutAddr)
{
	FInternetAddrBSD& BSDAddr = static_cast<FInternetAddrBSD&>(OutAddr);
	SOCKLEN Size = sizeof(sockaddr_storage);

	// Figure out what ip/port we are bound to
	bool bOk = getsockname(Socket, (sockaddr*)&BSDAddr.Addr, &Size) == 0;

	if (bOk == false)
	{
		check(SocketSubsystem);
		UE_LOGF(LogSockets, Error, "Failed to read address for socket (%ls)", SocketSubsystem->GetSocketError());
	}
}

bool FSocketBSD::GetPeerAddress(FInternetAddr& OutAddr)
{
	FInternetAddrBSD& BSDAddr = static_cast<FInternetAddrBSD&>(OutAddr);
	SOCKLEN Size = sizeof(sockaddr_storage);

	// Figure out what ip/port we are bound to
	int Result = getpeername(Socket, (sockaddr*)&BSDAddr.Addr, &Size);

	if (Result != 0)
	{
		check(SocketSubsystem);
		UE_LOGF(LogSockets, Warning, "Failed to read address for socket (%ls) with error %d", SocketSubsystem->GetSocketError(), Result);
	}
	return Result == 0;
}

bool FSocketBSD::SetNonBlocking(bool bIsNonBlocking)
{
#if PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS
	u_long Value = bIsNonBlocking ? true : false;
	return ioctlsocket(Socket,FIONBIO,&Value) == 0;
#else
	int Flags = fcntl(Socket, F_GETFL, 0);
	//Set the flag or clear it, without destroying the other flags.
	Flags = bIsNonBlocking ? Flags | O_NONBLOCK : Flags ^ (Flags & O_NONBLOCK);
	int err = fcntl(Socket, F_SETFL, Flags);
	return (err == 0 ? true : false);
#endif
}

bool FSocketBSD::SetNoDelay(bool bIsNoDelay)
{
	// Only do this if we are TCP.
	if (GetSocketType() == SOCKTYPE_Streaming)
	{
#if PLATFORM_HAS_BSD_SOCKET_FEATURE_NODELAY
		int Param = bIsNoDelay ? 1 : 0;
		return setsockopt(Socket, IPPROTO_TCP, TCP_NODELAY, (char*)&Param, sizeof(Param)) == 0;
#endif
	}

	return true;
}

bool FSocketBSD::SetBroadcast(bool bAllowBroadcast)
{
	int Param = bAllowBroadcast ? 1 : 0;
	return setsockopt(Socket,SOL_SOCKET,SO_BROADCAST,(char*)&Param,sizeof(Param)) == 0;
}

bool FSocketBSD::JoinMulticastGroup(const FInternetAddr& GroupAddress)
{
	const FInternetAddrBSD& BSDAddr = static_cast<const FInternetAddrBSD&>(GroupAddress);

#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	if (BSDAddr.GetProtocolType() == FNetworkProtocolTypes::IPv6)
	{
		ipv6_mreq imr;
		imr.ipv6mr_interface = htonl(BSDAddr.GetScopeId());
		imr.ipv6mr_multiaddr = ((sockaddr_in6*)&(BSDAddr.Addr))->sin6_addr;
		return (setsockopt(Socket, IPPROTO_IPV6, IPV6_JOIN_GROUP, (char*)&imr, sizeof(imr)) == 0);
	}
#endif

	ip_mreq imr;
	imr.imr_interface.s_addr = INADDR_ANY;
	imr.imr_multiaddr = ((sockaddr_in*)&(BSDAddr.Addr))->sin_addr;
	return (setsockopt(Socket, IPPROTO_IP, IP_ADD_MEMBERSHIP, (char*)&imr, sizeof(imr)) == 0);
}

bool FSocketBSD::JoinMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress)
{
	const FInternetAddrBSD& BSDAddr = static_cast<const FInternetAddrBSD&>(GroupAddress);
	const FInternetAddrBSD& BSDIFAddr = static_cast<const FInternetAddrBSD&>(InterfaceAddress);

#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	if (BSDAddr.GetProtocolType() == FNetworkProtocolTypes::IPv6)
	{
		ipv6_mreq imr;
		imr.ipv6mr_interface = htonl(BSDIFAddr.GetScopeId());
		imr.ipv6mr_multiaddr = ((sockaddr_in6*)&(BSDAddr.Addr))->sin6_addr;
		return (setsockopt(Socket, IPPROTO_IPV6, IPV6_JOIN_GROUP, (char*)&imr, sizeof(imr)) == 0);
	}
#endif

	ip_mreq imr;
	imr.imr_interface.s_addr = ((sockaddr_in*)&(BSDIFAddr.Addr))->sin_addr.s_addr;
	imr.imr_multiaddr = ((sockaddr_in*)&(BSDAddr.Addr))->sin_addr;
	return (setsockopt(Socket, IPPROTO_IP, IP_ADD_MEMBERSHIP, (char*)&imr, sizeof(imr)) == 0);
}

bool FSocketBSD::LeaveMulticastGroup(const FInternetAddr& GroupAddress)
{
	const FInternetAddrBSD& BSDAddr = static_cast<const FInternetAddrBSD&>(GroupAddress);

#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	if (BSDAddr.GetProtocolType() == FNetworkProtocolTypes::IPv6)
	{
		ipv6_mreq imr;
		imr.ipv6mr_interface = htonl(BSDAddr.GetScopeId());
		imr.ipv6mr_multiaddr = ((sockaddr_in6*)&(BSDAddr.Addr))->sin6_addr;
		return (setsockopt(Socket, IPPROTO_IPV6, IPV6_LEAVE_GROUP, (char*)&imr, sizeof(imr)) == 0);
	}
#endif

	ip_mreq imr;
	imr.imr_interface.s_addr = INADDR_ANY;
	imr.imr_multiaddr = ((sockaddr_in*)&(BSDAddr.Addr))->sin_addr;
	return (setsockopt(Socket, IPPROTO_IP, IP_DROP_MEMBERSHIP, (char*)&imr, sizeof(imr)) == 0);
}

bool FSocketBSD::LeaveMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress)
{
	const FInternetAddrBSD& BSDAddr = static_cast<const FInternetAddrBSD&>(GroupAddress);
	const FInternetAddrBSD& BSDIFAddr = static_cast<const FInternetAddrBSD&>(InterfaceAddress);

#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	if (BSDAddr.GetProtocolType() == FNetworkProtocolTypes::IPv6)
	{
		ipv6_mreq imr;
		imr.ipv6mr_interface = htonl(BSDIFAddr.GetScopeId());
		imr.ipv6mr_multiaddr = ((sockaddr_in6*)&(BSDAddr.Addr))->sin6_addr;
		return (setsockopt(Socket, IPPROTO_IPV6, IPV6_LEAVE_GROUP, (char*)&imr, sizeof(imr)) == 0);
	}
#endif

	ip_mreq imr;
	imr.imr_interface.s_addr = ((sockaddr_in*)&(BSDIFAddr.Addr))->sin_addr.s_addr;
	imr.imr_multiaddr = ((sockaddr_in*)&(BSDAddr.Addr))->sin_addr;
	return (setsockopt(Socket, IPPROTO_IP, IP_DROP_MEMBERSHIP, (char*)&imr, sizeof(imr)) == 0);
}

bool FSocketBSD::SetMulticastLoopback(bool bLoopback)
{
#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	if (SocketProtocol == FNetworkProtocolTypes::IPv6)
	{
		uint32 ShouldLoopback = bLoopback ? 1 : 0;
		return (setsockopt(Socket, IPPROTO_IPV6, IP_MULTICAST_LOOP, (char*)&ShouldLoopback, sizeof(ShouldLoopback)) == 0);
	}
#endif

	return (setsockopt(Socket, IPPROTO_IP, IP_MULTICAST_LOOP, (char*)&bLoopback, sizeof(bLoopback)) == 0);
}

bool FSocketBSD::SetMulticastTtl(uint8 TimeToLive)
{
#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	if (SocketProtocol == FNetworkProtocolTypes::IPv6)
	{
		uint32 RealTimeToLive = TimeToLive;
		return (setsockopt(Socket, IPPROTO_IPV6, IPV6_MULTICAST_HOPS, (char*)&RealTimeToLive, sizeof(RealTimeToLive)) == 0);
	}
#endif

	return (setsockopt(Socket, IPPROTO_IP, IP_MULTICAST_TTL, (char*)&TimeToLive, sizeof(TimeToLive)) == 0);
}

bool FSocketBSD::SetMulticastInterface(const FInternetAddr& InterfaceAddress)
{
	const FInternetAddrBSD& BSDIFAddr = static_cast<const FInternetAddrBSD&>(InterfaceAddress);

#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	if (BSDIFAddr.GetProtocolType() == FNetworkProtocolTypes::IPv6)
	{
		uint32 InterfaceIndex = htonl(BSDIFAddr.GetScopeId());
		return (setsockopt(Socket, IPPROTO_IPV6, IPV6_MULTICAST_IF, (char*)&InterfaceIndex, sizeof(InterfaceIndex)) == 0);
	}
#endif

	in_addr InterfaceAddr = ((sockaddr_in*)&(BSDIFAddr.Addr))->sin_addr;
	return (setsockopt(Socket, IPPROTO_IP, IP_MULTICAST_IF, (char*)&InterfaceAddr, sizeof(InterfaceAddr)) == 0);
}

bool FSocketBSD::SetReuseAddr(bool bAllowReuse)
{
	int Param = bAllowReuse ? 1 : 0;
	int ReuseAddrResult = setsockopt(Socket, SOL_SOCKET, SO_REUSEADDR, (char*)&Param, sizeof(Param));
#ifdef SO_REUSEPORT // Linux kernel 3.9+ and FreeBSD define this separately
	if (ReuseAddrResult == 0)
	{
		return setsockopt(Socket, SOL_SOCKET, SO_REUSEPORT, (char *)&Param, sizeof(Param)) == 0;
	}
#endif
	return ReuseAddrResult == 0;
}

bool FSocketBSD::SetLinger(bool bShouldLinger,int32 Timeout)
{
	linger ling;

	ling.l_onoff = bShouldLinger;
	// The type of l_linger varies by platform.
	ling.l_linger = IntCastChecked<decltype(ling.l_linger)>(Timeout);

	return setsockopt(Socket,SOL_SOCKET,SO_LINGER,(char*)&ling,sizeof(ling)) == 0;
}

bool FSocketBSD::SetRecvErr(bool bUseErrorQueue)
{
	// Not supported, but return true to avoid spurious log messages
	return true;
}

bool FSocketBSD::SetSendBufferSize(int32 Size,int32& NewSize)
{
	SOCKLEN SizeSize = sizeof(int32);
	bool bOk = setsockopt(Socket,SOL_SOCKET,SO_SNDBUF,(char*)&Size,sizeof(int32)) == 0;

	// Read the value back in case the size was modified
	getsockopt(Socket,SOL_SOCKET,SO_SNDBUF,(char*)&NewSize, &SizeSize);

	return bOk;
}

bool FSocketBSD::SetReceiveBufferSize(int32 Size,int32& NewSize)
{
	SOCKLEN SizeSize = sizeof(int32);
	bool bOk = setsockopt(Socket,SOL_SOCKET,SO_RCVBUF,(char*)&Size,sizeof(int32)) == 0;

	// Read the value back in case the size was modified
	getsockopt(Socket,SOL_SOCKET,SO_RCVBUF,(char*)&NewSize, &SizeSize);

	return bOk;
}

bool FSocketBSD::SetSendSegmentationOffloadSize(uint16 InSegmentPayloadBytes)
{
#if PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS
	uint32 OptSegmentSize = InSegmentPayloadBytes;
	return 0 == setsockopt(Socket, IPPROTO_UDP, UDP_SEND_MSG_SIZE, (char*)&OptSegmentSize, sizeof(OptSegmentSize));
#else
	return false;
#endif
}

int32 FSocketBSD::GetPortNo(void)
{
	sockaddr_storage Addr;
	SOCKLEN Size = sizeof(sockaddr_storage);

	// Figure out what ip/port we are bound to
	bool bOk = getsockname(Socket, (sockaddr*)&Addr, &Size) == 0;
	
	if (bOk == false)
	{
		check(SocketSubsystem);
		UE_LOGF(LogSockets, Error, "Failed to read address for socket (%ls)", SocketSubsystem->GetSocketError());
		
		return 0;
	}

#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	if (Addr.ss_family == AF_INET6)
	{
		return ntohs(((sockaddr_in6&)Addr).sin6_port);
	}
#endif

	// Convert big endian port to native endian port.
	return ntohs(((sockaddr_in&)Addr).sin_port);
}

bool FSocketBSD::SetIpPktInfo(bool bEnable)
{
#ifdef IP_PKTINFO
	int Opt = bEnable ? 1 : 0;
	if (setsockopt(Socket, IPPROTO_IP, IP_PKTINFO, (char*)&Opt, sizeof(Opt)) != 0)
	{
		UE_LOGF(LogSockets, Error, "Failed to set sock IP_PKTINFO for socket (%ls)", SocketSubsystem->GetSocketError());
		return false;
	}
	return true;
#else
	return false;
#endif
}

bool FSocketBSD::SetIPv6Only(bool bIPv6Only)
{
#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	if (GetProtocol() == FNetworkProtocolTypes::IPv6)
	{
		int v6only = bIPv6Only ? 1 : 0;
		bool bOk = (setsockopt(Socket, IPPROTO_IPV6, IPV6_V6ONLY, (char*)&v6only, sizeof(v6only)) == 0);

		if (bOk == false)
		{
			check(SocketSubsystem);
			UE_LOGF(LogSockets, Error, "Failed to set sock opt for socket (%ls)", SocketSubsystem->GetSocketError());
		}

		return bOk;
	}
	else
	{
		return true;
	}
#else
	return false;
#endif
}


/* FSocketBSD implementation
*****************************************************************************/

ESocketBSDReturn FSocketBSD::HasState(ESocketBSDParam State, FTimespan WaitTime)
{
#if PLATFORM_HAS_BSD_SOCKET_FEATURE_POLL && USE_SOCKET_FEATURE_POLL
	struct pollfd FDSet;
	FDSet.fd = Socket;
	FDSet.revents = 0;

	switch (State)
	{
	case ESocketBSDParam::CanRead:
		FDSet.events = POLLIN;
		break;

	case ESocketBSDParam::CanWrite:
		FDSet.events = POLLOUT;
		break;

	case ESocketBSDParam::HasError:
		FDSet.events = POLLERR | POLLPRI;
		break;
	}
	int Result = ::poll(&FDSet, 1, (int)WaitTime.GetTotalMilliseconds());
	if (Result >= 0)
	{
		if ((FDSet.revents & FDSet.events) > 0)
		{
			return ESocketBSDReturn::Yes;
		}
		else
		{
			return ESocketBSDReturn::No;
		}
	}
	else
	{
		return ESocketBSDReturn::EncounteredError;
	}

#elif PLATFORM_HAS_BSD_SOCKET_FEATURE_SELECT
	// convert WaitTime to a timeval
	timeval Time;
	Time.tv_sec = (int32)WaitTime.GetTotalSeconds();
	Time.tv_usec = WaitTime.GetFractionMicro();

	fd_set SocketSet;

	// Set up the socket sets we are interested in (just this one)
	FD_ZERO(&SocketSet);
	FD_SET(Socket, &SocketSet);

	timeval* TimePointer = WaitTime.GetTicks() >= 0 ? &Time : nullptr;

	// Check the status of the state
	int32 SelectStatus = 0;
	switch (State)
	{
	case ESocketBSDParam::CanRead:
		SelectStatus = select(IntCastChecked<int>(Socket + 1), &SocketSet, NULL, NULL, TimePointer);
		break;

	case ESocketBSDParam::CanWrite:
		SelectStatus = select(IntCastChecked<int>(Socket + 1), NULL, &SocketSet, NULL, TimePointer);
		break;

	case ESocketBSDParam::HasError:
		SelectStatus = select(IntCastChecked<int>(Socket + 1), NULL, NULL, &SocketSet, TimePointer);
		break;
	}

	// if the select returns a positive number, the socket had the state, 0 means didn't have it, and negative is API error condition (not socket's error state)
	return SelectStatus > 0 ? ESocketBSDReturn::Yes :
		SelectStatus == 0 ? ESocketBSDReturn::No :
		ESocketBSDReturn::EncounteredError;
#else
	UE_LOGF(LogSockets, Fatal, "This platform doesn't support poll() or select(), but FSocketBSD::HasState was not overridden");
	return ESocketBSDReturn::EncounteredError;
#endif
}


#endif	//PLATFORM_HAS_BSD_SOCKETS
