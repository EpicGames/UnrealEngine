// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/PlatformTime.h"
#include "BSDSockets/SocketSubsystemBSDPrivate.h"

#include <atomic>

class FInternetAddr;

#if PLATFORM_HAS_BSD_SOCKETS || PLATFORM_HAS_BSD_IPV6_SOCKETS

#include "Sockets.h"


/**
 * Enumerates BSD socket state parameters.
 */
enum class ESocketBSDParam : uint8
{
	CanRead,
	CanWrite,
	HasError,
};


/**
 * Enumerates BSD socket state return values.
 */
enum class ESocketBSDReturn : uint8
{
	Yes,
	No,
	EncounteredError,
};


/**
 * Implements a BSD network socket.
 */
class FSocketBSD
	: public FSocket
{
public:

	/**
	 * Assigns a BSD socket to this object.
	 *
	 * @param InSocket the socket to assign to this object.
	 * @param InSocketType the type of socket that was created.
	 * @param InSocketDescription the debug description of the socket.
	 * @param InSocketProtocol the protocol this socket is initialized with
	 */
	FSocketBSD(SOCKET InSocket, ESocketType InSocketType, const FString& InSocketDescription, const FName& InSocketProtocol, ISocketSubsystem * InSubsystem)
		: FSocket(InSocketType, InSocketDescription, InSocketProtocol)
		, Socket(InSocket)
		, LastActivityTime(0.0)
		, SocketSubsystem(InSubsystem)
	{ }

	/**
	 * Destructor.
	 *
	 * Closes the socket if it is still open
	 */
	virtual ~FSocketBSD()
	{
		FSocketBSD::Close();
	}

public:

	/**
	* Gets the Socket for anyone who knows they have an FSocketBSD.
	*
	* @return The native socket.
	*/
	SOCKET GetNativeSocket()
	{
		return Socket;
	}

public:

	// FSocket overrides

	virtual bool Shutdown(ESocketShutdownMode Mode) override;
	virtual bool Close() override;
	virtual bool Bind(const FInternetAddr& Addr) override;
	virtual bool Connect(const FInternetAddr& Addr) override;
	virtual bool Listen(int32 MaxBacklog) override;
	virtual bool WaitForPendingConnection(bool& bHasPendingConnection, const FTimespan& WaitTime) override;
	virtual bool HasPendingData(uint32& PendingDataSize) override;
	virtual class FSocket* Accept(const FString& SocketDescription) override;
	virtual class FSocket* Accept(FInternetAddr& OutAddr, const FString& SocketDescription) override;
	virtual bool SendTo(const uint8* Data, int32 Count, int32& BytesSent, const FInternetAddr& Destination) override;
	virtual bool Send(const uint8* Data, int32 Count, int32& BytesSent) override;
	virtual bool RecvFrom(uint8* Data, int32 BufferSize, int32& BytesRead, FInternetAddr& Source, ESocketReceiveFlags::Type Flags = ESocketReceiveFlags::None) override;
	virtual bool Recv(uint8* Data,int32 BufferSize,int32& BytesRead, ESocketReceiveFlags::Type Flags = ESocketReceiveFlags::None) override;
	virtual bool Wait(ESocketWaitConditions::Type Condition, FTimespan WaitTime) override;
	virtual ESocketConnectionState GetConnectionState() override;
	virtual void GetAddress(FInternetAddr& OutAddr) override;
	virtual bool GetPeerAddress(FInternetAddr& OutAddr) override;
	virtual bool SetNonBlocking(bool bIsNonBlocking = true) override;
	virtual bool SetNoDelay(bool bIsNoDelay = true) override;
	virtual bool SetBroadcast(bool bAllowBroadcast = true) override;
	virtual bool JoinMulticastGroup(const FInternetAddr& GroupAddress) override;
	virtual bool JoinMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress) override;
	virtual bool LeaveMulticastGroup(const FInternetAddr& GroupAddress) override;
	virtual bool LeaveMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress) override;
	virtual bool SetMulticastLoopback(bool bLoopback) override;
	virtual bool SetMulticastTtl(uint8 TimeToLive) override;
	virtual bool SetMulticastInterface(const FInternetAddr& InterfaceAddress) override;
	virtual bool SetReuseAddr(bool bAllowReuse = true) override;
	virtual bool SetLinger(bool bShouldLinger = true, int32 Timeout = 0) override;
	virtual bool SetRecvErr(bool bUseErrorQueue = true) override;
	virtual bool SetSendBufferSize(int32 Size,int32& NewSize) override;
	virtual bool SetReceiveBufferSize(int32 Size,int32& NewSize) override;
	virtual bool SetSendSegmentationOffloadSize(uint16 InSegmentPayloadBytes) override;
	virtual int32 GetPortNo() override;
	virtual bool SetIpPktInfo(bool bEnable) override;
	virtual bool RecvFromWithPktInfo(uint8* Data, int32 BufferSize, int32& BytesRead, FInternetAddr& Source, FInternetAddr& Destination, ESocketReceiveFlags::Type Flags = ESocketReceiveFlags::None) override;
	[[nodiscard]] virtual UPTRINT ReleaseNativeSocket() override;

	bool SetIPv6Only(bool bIPv6Only);

protected:

	/** This is generally select(), but makes it easier for platforms without select to replace it. */
	virtual ESocketBSDReturn HasState(ESocketBSDParam State, FTimespan WaitTime = FTimespan::Zero());

	/** Updates this socket's time of last activity. */
	FORCEINLINE void UpdateActivity()
	{
		LastActivityTime = FPlatformTime::Seconds();
	}

	/** Holds the BSD socket object. */
	SOCKET Socket;

	/** Last activity time. */
	std::atomic<double> LastActivityTime { 0.0 };

	/** Flags which will be passed to send and sendto. */
	int SendFlags = 0;

	/** Pointer to the subsystem that created it. */
	ISocketSubsystem* SocketSubsystem;
};


#endif	//PLATFORM_HAS_BSD_SOCKETS
