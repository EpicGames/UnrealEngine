// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SocketSubsystemIOS.h"
#include "BSDSockets/SocketSubsystemBSDPrivate.h"
#include "BSDSockets/IPAddressBSD.h"

#if PLATFORM_HAS_BSD_IPV6_SOCKETS

class FInternetAddrBSDIOS : public FInternetAddrBSD
{
public:

	FInternetAddrBSDIOS(FSocketSubsystemBSD* InSocketSubsystem, FName RequestedProtocol = NAME_None) : 
		FInternetAddrBSD(InSocketSubsystem, RequestedProtocol)
	{
	}

	/** Sets the address to broadcast */
	virtual void SetIPv6BroadcastAddress() override
	{
		FSocketSubsystemIOS* SocketSubsystemIOS = static_cast<FSocketSubsystemIOS*>(SocketSubsystem);
		if (SocketSubsystemIOS)
		{
			TSharedPtr<FInternetAddrBSDIOS> MultiCastAddr = StaticCastSharedPtr<FInternetAddrBSDIOS>(SocketSubsystemIOS->GetAddressFromString(TEXT("ff02::1")));
			if (!MultiCastAddr.IsValid())
			{
				UE_LOGF(LogSockets, Warning, "Could not resolve the broadcast address for iOS, this address will just be blank");
			}
			else
			{
				// Set the address from the query
				SetRawIp(MultiCastAddr->GetRawIp());

				// Grab the scope id to set the broadcast address to
				// We could potentially do this with less loops but it's best to make sure that we set an IPv6 scope
				// And iOS could return an IPv4 address if we just called GetLocalHostAddr
				TArray<TSharedRef<FInternetAddr>> BindingAddresses = SocketSubsystemIOS->GetLocalBindAddresses();
				if (BindingAddresses.Num() > 0)
				{
					for (const auto& BindAddress : BindingAddresses)
					{
						if (BindAddress->GetProtocolType() == FNetworkProtocolTypes::IPv6)
						{
							uint32 ScopeId = StaticCastSharedRef<FInternetAddrBSDIOS>(BindAddress)->GetScopeId();
							SetScopeId(ScopeId);
							break;
						}
					}
				}
				else
				{
					UE_LOGF(LogSockets, Warning, "Could not get binding addresses to set the internal scope id for the broadcast address");
				}
			}
		}
		else
		{
			UE_LOGF(LogSockets, Warning, "Could not get the socketsubsystem for querying the scope id of the broadcast address");
		}
		SetPort(0);
	}
};

#endif
