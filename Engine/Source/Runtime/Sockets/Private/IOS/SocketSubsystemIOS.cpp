// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocketSubsystemIOS.h"
#include "SocketSubsystemModule.h"
#include "Modules/ModuleManager.h"
#include "SocketsBSDIOS.h"
#include "IPAddressBSDIOS.h"
#include <net/if.h>
#include <ifaddrs.h>

FSocketSubsystemIOS* FSocketSubsystemIOS::SocketSingleton = NULL;

class FSocketBSD* FSocketSubsystemIOS::InternalBSDSocketFactory(SOCKET Socket, ESocketType SocketType, const FString& SocketDescription, const FName& SocketProtocol)
{
	UE_LOGF(LogIOS, Log, " FSocketSubsystemIOS::InternalBSDSocketFactory");
	return new FSocketBSDIOS(Socket, SocketType, SocketDescription, SocketProtocol, this);
}

FName CreateSocketSubsystem( FSocketSubsystemModule& SocketSubsystemModule )
{
	FName SubsystemName("IOS");
	// Create and register our singleton factor with the main online subsystem for easy access
	FSocketSubsystemIOS* SocketSubsystem = FSocketSubsystemIOS::Create();
	FString Error;
	if (SocketSubsystem->Init(Error))
	{
		SocketSubsystemModule.RegisterSocketSubsystem(SubsystemName, SocketSubsystem);
		return SubsystemName;
	}
	else
	{
		FSocketSubsystemIOS::Destroy();
		return NAME_None;
	}
}

void DestroySocketSubsystem( FSocketSubsystemModule& SocketSubsystemModule )
{
	SocketSubsystemModule.UnregisterSocketSubsystem(FName("IOS"));
	FSocketSubsystemIOS::Destroy();
}

FSocketSubsystemIOS* FSocketSubsystemIOS::Create()
{
	if (SocketSingleton == NULL)
	{
		SocketSingleton = new FSocketSubsystemIOS();
	}

	return SocketSingleton;
}

void FSocketSubsystemIOS::Destroy()
{
	if (SocketSingleton != NULL)
	{
		SocketSingleton->Shutdown();
		delete SocketSingleton;
		SocketSingleton = NULL;
	}
}

bool FSocketSubsystemIOS::Init(FString& Error)
{
	return true;
}

void FSocketSubsystemIOS::Shutdown(void)
{
}


bool FSocketSubsystemIOS::HasNetworkDevice()
{
	return true;
}

FSocket* FSocketSubsystemIOS::CreateSocket(const FName& SocketType, const FString& SocketDescription, const FName& ProtocolType)
{
	FSocketBSD* NewSocket = (FSocketBSD*)FSocketSubsystemBSD::CreateSocket(SocketType, SocketDescription, ProtocolType);
	if (NewSocket)
	{
		NewSocket->SetIPv6Only(false);

		// disable the SIGPIPE exception 
		int bAllow = 1;
		setsockopt(NewSocket->GetNativeSocket(), SOL_SOCKET, SO_NOSIGPIPE, &bAllow, sizeof(bAllow));
	}
	return NewSocket;
}

/** Priority values for iOS adapters. */
enum class EAdapterPriorityValues : uint8
{
	None = 0,
	IPv6Wifi = 1,
	IPv4Wifi = 2,
	IPv6Cell = 3,
	IPv4Cell = 4,
};

/** Helper struct to sort our adapters based off of interface information */
struct FSortedPriorityAddresses
{
	FSortedPriorityAddresses(FSocketSubsystemIOS* SocketSub) :
		Priority(EAdapterPriorityValues::None)
	{
		Address = StaticCastSharedRef<FInternetAddrBSDIOS>(SocketSub->CreateInternetAddr());
	}

	TSharedPtr<FInternetAddrBSDIOS> Address;
	EAdapterPriorityValues Priority;
	FString InterfaceName;

	bool operator<(const FSortedPriorityAddresses& Other) const
	{
		// Lower values are better than higher values
		return Priority < Other.Priority;
	}

	FString ToString() const
	{
		FString AdapterPriorityValue;
		switch (Priority)
		{
		default:
		case EAdapterPriorityValues::None:
			AdapterPriorityValue = TEXT("Invalid/Any");
			break;
		case EAdapterPriorityValues::IPv6Wifi:
			AdapterPriorityValue = TEXT("IPv6 Wifi");
			break;
		case EAdapterPriorityValues::IPv6Cell:
			AdapterPriorityValue = TEXT("IPv6 Cell");
			break;
		case EAdapterPriorityValues::IPv4Wifi:
			AdapterPriorityValue = TEXT("IPv4 Wifi");
			break;
		case EAdapterPriorityValues::IPv4Cell:
			AdapterPriorityValue = TEXT("IPv4 Cell");
			break;
		}

		return FString::Printf(TEXT("%s [Interface: %s] %s (%d)"), *Address->ToString(true), *InterfaceName, *AdapterPriorityValue, (int32)Priority);
	}
};

bool FSocketSubsystemIOS::GetLocalAdapterAddresses(TArray<TSharedPtr<FInternetAddr>>& OutAddresses)
{
	// Since getifaddrs does not return the addresses in a sorted priority list, we need to do that ourselves.
	// This does lead to us processing a lot of information, however we get a massive benefit in making sure the adapters we use are the best
	TArray<FSortedPriorityAddresses> SortedAddresses;

	// Grow the array to a largish size. We don't know how many interfaces we could have, and there's no way to tell ahead of time.
	SortedAddresses.Reserve(12);

	// Get all of the addresses this device has
	ifaddrs* Interfaces = nullptr;
	if (getifaddrs(&Interfaces) == 0)
	{
		// Loop through linked list of interfaces
		for (ifaddrs* Travel = Interfaces; Travel != nullptr; Travel = Travel->ifa_next)
		{
			// Skip over anything that does not have valid data, is the loopback interface, or is currently not in use
			if (Travel->ifa_addr == nullptr || (Travel->ifa_flags & (IFF_LOOPBACK)) != 0 || (Travel->ifa_flags & (IFF_RUNNING | IFF_UP)) == 0)
			{
				continue;
			}

			// Also drop any interface that's not a wifi or mobile adapter
			bool bIsWifi = strncmp(Travel->ifa_name, "en", 2) == 0;
			bool bIsMobile = strncmp(Travel->ifa_name, "pdp_ip", 6) == 0;
			if (!bIsWifi && !bIsMobile)
			{
				continue;
			}

			// Otherwise, this is an address we can consider.
			FSortedPriorityAddresses NewAddress(this);
			sockaddr_storage* AddrData = reinterpret_cast<sockaddr_storage*>(Travel->ifa_addr);

			// Set in the address data.
			NewAddress.Address->SetIp(*AddrData);
			// Make sure to set the scope id as it will not be in the AddrData by default.
			NewAddress.Address->SetScopeId(ntohl(if_nametoindex(Travel->ifa_name)));

			// Ignore anything that is not a valid address that we can use.
			if (NewAddress.Address->IsValid())
			{
				// Assign priority as needed. IPv6 is always better than IPv4, and WIFI is better than Cellular
				if (Travel->ifa_addr->sa_family == AF_INET6)
				{
					NewAddress.Priority = (bIsWifi) ?
						EAdapterPriorityValues::IPv6Wifi : EAdapterPriorityValues::IPv6Cell;
				}
				else if (Travel->ifa_addr->sa_family == AF_INET)
				{
					NewAddress.Priority = (bIsWifi) ?
						EAdapterPriorityValues::IPv4Wifi : EAdapterPriorityValues::IPv4Cell;
				}
				else
				{
					continue;
				}

				NewAddress.InterfaceName = ANSI_TO_TCHAR(Travel->ifa_name);
				UE_LOGF(LogIOS, Verbose, "Added address %ls with interface %ls", *NewAddress.ToString(), *NewAddress.InterfaceName);
				SortedAddresses.Add(NewAddress);
			}
		}

		// Free interface memory
		freeifaddrs(Interfaces);

		// Sort the addresses based on priority now that all of them are added
		SortedAddresses.Sort();

		// With the now sorted list, add them to our output addresses.
		for (const auto& AddressInterface : SortedAddresses)
		{
			UE_LOGF(LogIOS, Verbose, "Ordered Address: %ls", *AddressInterface.ToString());
			OutAddresses.Add(AddressInterface.Address);
		}

		return OutAddresses.Num() > 0;
	}
	return false;
}

TArray<TSharedRef<FInternetAddr>> FSocketSubsystemIOS::GetLocalBindAddresses()
{
	TArray<TSharedRef<FInternetAddr>> BindingAddresses = FSocketSubsystemBSD::GetLocalBindAddresses();

	uint32 AdapterScopeId = 0;
	TSharedRef<FInternetAddr> MultihomeAddr = CreateInternetAddr();
	if (!GetMultihomeAddress(MultihomeAddr))
	{
		TArray<TSharedPtr<FInternetAddr>> AdapterAddresses;
		if (GetLocalAdapterAddresses(AdapterAddresses))
		{
			AdapterScopeId = StaticCastSharedPtr<FInternetAddrBSDIOS>(AdapterAddresses[0])->GetScopeId();
		}
		else
		{
			UE_LOGF(LogIOS, Warning, "Unable to grab the local adapters on this device in order to write scope id on binding addreesses!");
		}
	}
	else
	{
		AdapterScopeId = StaticCastSharedRef<FInternetAddrBSDIOS>(MultihomeAddr)->GetScopeId();
	}

	UE_LOGF(LogIOS, Verbose, "Using scope id %u for binding addresses", AdapterScopeId);

	// For iOS, we pull the best scope id that we know of and use that to bind with.
	for (auto& BindAddress : BindingAddresses)
	{
		StaticCastSharedRef<FInternetAddrBSDIOS>(BindAddress)->SetScopeId(AdapterScopeId);
	}

	return BindingAddresses;
}

TSharedRef<FInternetAddr> FSocketSubsystemIOS::CreateInternetAddr()
{
	return MakeShareable(new FInternetAddrBSDIOS(this));
}

TSharedRef<FInternetAddr> FSocketSubsystemIOS::CreateInternetAddr(const FName RequiredProtocol)
{
	return MakeShareable(new FInternetAddrBSDIOS(this, RequiredProtocol));
}
