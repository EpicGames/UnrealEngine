// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SocketSubsystem.h"
#include "BSDSockets/SocketSubsystemBSD.h"
#include "SocketSubsystemPackage.h"

/**
 * iOS specific socket subsystem implementation
 */
class FSocketSubsystemIOS : public FSocketSubsystemBSD
{
protected:
	/** Single instantiation of this subsystem */
	static FSocketSubsystemIOS* SocketSingleton;

	/** Whether Init() has been called before or not */
	bool bTriedToInit;

	// @todo ios: This is kind of hacky, since there's no UBT that should set PACKAGE_SCOPE
// PACKAGE_SCOPE:
public:

	/** 
	 * Singleton interface for this subsystem 
	 * @return the only instance of this subsystem
	 */
	static FSocketSubsystemIOS* Create();

	/**
	 * Performs iOS specific socket clean up
	 */
	static void Destroy();


	virtual FName GetDefaultSocketProtocolFamily() const override
	{
		return FNetworkProtocolTypes::IPv6;
	}

public:

	FSocketSubsystemIOS() 
		: bTriedToInit(false)
	{
	}

	virtual ~FSocketSubsystemIOS()
	{
	}

	virtual bool Init(FString& Error) override;
	virtual void Shutdown() override;
	virtual bool HasNetworkDevice() override;
	virtual FSocket* CreateSocket(const FName& SocketType, const FString& SocketDescription, const FName& ProtocolType) override;
	virtual TSharedRef<FInternetAddr> CreateInternetAddr() override;
	virtual TSharedRef<FInternetAddr> CreateInternetAddr(const FName RequiredProtocol) override;
	virtual bool GetLocalAdapterAddresses(TArray<TSharedPtr<FInternetAddr>>& OutAddresses) override;
	virtual TArray<TSharedRef<FInternetAddr>> GetLocalBindAddresses() override;
	virtual class FSocketBSD* InternalBSDSocketFactory(SOCKET Socket, ESocketType SocketType, const FString& SocketDescription, const FName& SocketProtocol) override;
};