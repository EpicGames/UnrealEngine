// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPAddressBSDIOS.h"
#include "BSDSockets/SocketSubsystemBSDPrivate.h"
#include "Sockets.h"
#include "BSDSockets/SocketsBSD.h"

class FInternetAddr;

/**
 * Implements a BSD network socket on IOS.
 */
class FSocketBSDIOS
	: public FSocketBSD
{
public:

	FSocketBSDIOS(SOCKET InSocket, ESocketType InSocketType, const FString& InSocketDescription, const FName& InSocketProtocol, ISocketSubsystem* InSubsystem)
		:FSocketBSD(InSocket, InSocketType, InSocketDescription, InSocketProtocol, InSubsystem)
	{
	}

	virtual ~FSocketBSDIOS()
	{
		FSocketBSD::Close();
	}

	virtual bool JoinMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress) override
	{
		const FInternetAddrBSDIOS& IOSInterfaceAddr = static_cast<const FInternetAddrBSDIOS&>(InterfaceAddress);
		// iOS does not allow usage of an interface id of 0, so redirect to using the group address instead
		if (GroupAddress.GetProtocolType() == FNetworkProtocolTypes::IPv6 && IOSInterfaceAddr.GetScopeId() == 0)
		{
			return FSocketBSD::JoinMulticastGroup(GroupAddress);
		}

		return FSocketBSD::JoinMulticastGroup(GroupAddress, InterfaceAddress);
	}

	virtual bool LeaveMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress) override
	{
		const FInternetAddrBSDIOS& IOSInterfaceAddr = static_cast<const FInternetAddrBSDIOS&>(InterfaceAddress);
		if (GroupAddress.GetProtocolType() == FNetworkProtocolTypes::IPv6 && IOSInterfaceAddr.GetScopeId() == 0)
		{
			return FSocketBSD::LeaveMulticastGroup(GroupAddress);
		}

		return FSocketBSD::LeaveMulticastGroup(GroupAddress, InterfaceAddress);
	}
};
