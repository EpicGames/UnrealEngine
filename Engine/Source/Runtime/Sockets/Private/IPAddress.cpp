// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPAddress.h"
#include "IPAddressAsyncResolve.h"
#include "SocketSubsystem.h"
#include "AddressInfoTypes.h"

void FInternetAddr::DumpAddrData() const
{
	UE_LOGF(LogSockets, Log, "Dumping Addr Data : Addr = %ls Protocol = %ls Port = %i PlatformPort = %i", *ToString(true), *GetProtocolType().ToString(), GetPort(), GetPlatformPort());
}

/**
 * Sets the address to return to the caller
 *
 * @param InAddr the address that is being cached
 */
FResolveInfoCached::FResolveInfoCached(const FInternetAddr& InAddr)
{
	Addr = InAddr.Clone();
}

/**
 * Copies the host name for async resolution
 *
 * @param InHostName the host name to resolve
 */
FResolveInfoAsync::FResolveInfoAsync(const ANSICHAR* InHostName) :
	ErrorCode(SE_NO_ERROR),
	bShouldAbandon(false),
	AsyncTask(this)
{
	FCStringAnsi::Strncpy(HostName,InHostName,256);
}

/**
 * Resolves the specified host name
 */
void FResolveInfoAsync::DoWork()
{
	int32 AttemptCount = 0;

	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();

	// Make up to 3 attempts to resolve it
	do 
	{
		FAddressInfoResult GAIResults = SocketSubsystem->GetAddressInfo(ANSI_TO_TCHAR(HostName), nullptr, EAddressInfoFlags::Default, NAME_None);
		ErrorCode = GAIResults.ReturnCode;
		if (ErrorCode != SE_NO_ERROR)
		{
			if (ErrorCode == SE_HOST_NOT_FOUND || ErrorCode == SE_NO_DATA || ErrorCode == SE_ETIMEDOUT)
			{
				// Force a failure
				AttemptCount = 3;
			}
		}
		else if(GAIResults.Results.Num() > 0)
		{
			
			// Pull out the address
			Addr = GAIResults.Results[0].Address;

			// Cache for reuse
			SocketSubsystem->AddHostNameToCache(HostName, Addr);

			return;
		}
		else
		{
			ErrorCode = SE_NO_DATA;
		}

		AttemptCount++;
	}
	while (ErrorCode != SE_NO_ERROR && AttemptCount < 3 && bShouldAbandon == false);
}
