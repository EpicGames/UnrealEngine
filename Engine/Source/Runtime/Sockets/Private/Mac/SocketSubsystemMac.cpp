// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocketSubsystemMac.h"
#include "SocketSubsystemModule.h"
#include "Modules/ModuleManager.h"
#include "BSDSockets/IPAddressBSD.h"

#include <ifaddrs.h>
#include <net/if.h>

FSocketSubsystemMac* FSocketSubsystemMac::SocketSingleton = NULL;

FName CreateSocketSubsystem( FSocketSubsystemModule& SocketSubsystemModule )
{
	FName SubsystemName("MAC");
	// Create and register our singleton factor with the main online subsystem for easy access
	FSocketSubsystemMac* SocketSubsystem = FSocketSubsystemMac::Create();
	FString Error;
	if (SocketSubsystem->Init(Error))
	{
		SocketSubsystemModule.RegisterSocketSubsystem(SubsystemName, SocketSubsystem);
		return SubsystemName;
	}
	else
	{
		FSocketSubsystemMac::Destroy();
		return NAME_None;
	}
}

void DestroySocketSubsystem( FSocketSubsystemModule& SocketSubsystemModule )
{
	SocketSubsystemModule.UnregisterSocketSubsystem(FName("MAC"));
	FSocketSubsystemMac::Destroy();
}

FSocketSubsystemMac* FSocketSubsystemMac::Create()
{
	if (SocketSingleton == NULL)
	{
		SocketSingleton = new FSocketSubsystemMac();
	}

	return SocketSingleton;
}

void FSocketSubsystemMac::Destroy()
{
	if (SocketSingleton != NULL)
	{
		SocketSingleton->Shutdown();
		delete SocketSingleton;
		SocketSingleton = NULL;
	}
}

bool FSocketSubsystemMac::Init(FString& Error)
{
	return true;
}

void FSocketSubsystemMac::Shutdown(void)
{
}


bool FSocketSubsystemMac::HasNetworkDevice()
{
	return true;
}

class FSocketBSD* FSocketSubsystemMac::InternalBSDSocketFactory(SOCKET Socket, ESocketType SocketType, const FString& SocketDescription, const FName& SocketProtocol)
{
	// return a new socket object
	FSocketMac* MacSock = new FSocketMac(Socket, SocketType, SocketDescription, SocketProtocol, this);
	if (MacSock)
	{
		// disable the SIGPIPE exception
		int bAllow = 1;
		setsockopt(Socket, SOL_SOCKET, SO_NOSIGPIPE, &bAllow, sizeof(bAllow));
	}
	return MacSock;
}

bool FSocketSubsystemMac::GetLocalAdapterAddresses(TArray<TSharedPtr<FInternetAddr>>& OutAddresses)
{
	ifaddrs* Interfaces = NULL;
	int InterfaceQueryRet = getifaddrs(&Interfaces);
	UE_LOGF(LogSockets, Verbose, "Querying net interfaces returned: %d", InterfaceQueryRet);
	const bool bDisableIPv6 = CVarDisableIPv6.GetValueOnAnyThread() == 1;
	if (InterfaceQueryRet == 0)
	{
		// Loop through linked list of interfaces
		for (ifaddrs* Travel = Interfaces; Travel != NULL; Travel = Travel->ifa_next)
		{
			// Skip over empty data sets.
			if (Travel->ifa_addr == NULL)
			{
				continue;
			}

			uint16 AddrFamily = Travel->ifa_addr->sa_family;
			// Find any up and non-loopback addresses
			if ((Travel->ifa_flags & IFF_UP) != 0 &&
				(Travel->ifa_flags & IFF_LOOPBACK) == 0 && 
				(AddrFamily == AF_INET || (!bDisableIPv6 && AddrFamily == AF_INET6)))
			{
				TSharedRef<FInternetAddrBSD> NewAddress = MakeShareable(new FInternetAddrBSD(this));
				NewAddress->SetIp(*((sockaddr_storage*)Travel->ifa_addr));
				uint32 AddressInterface = ntohl(if_nametoindex(Travel->ifa_name));

				NewAddress->SetScopeId(AddressInterface);
				OutAddresses.Add(NewAddress);
				UE_LOGF(LogSockets, Verbose, "Added address %ls on interface %d", *(NewAddress->ToString(false)), AddressInterface);
			}
		}

		freeifaddrs(Interfaces);
	}
	else
	{
		UE_LOGF(LogSockets, Warning, "getifaddrs returned result %d", InterfaceQueryRet);
		return false;
	}

	return (OutAddresses.Num() > 0);
}
