// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SocketSubsystem.h"
#include "BSDSockets/SocketSubsystemBSD.h"
#include "BSDSockets/SocketsBSD.h"
#include "SocketSubsystemPackage.h"

/**
 * Mac specific socket subsystem implementation
 */
class FSocketSubsystemMac : public FSocketSubsystemBSD
{
protected:

	/** Single instantiation of this subsystem */
	static FSocketSubsystemMac* SocketSingleton;

	/** Whether Init() has been called before or not */
	bool bTriedToInit;

PACKAGE_SCOPE:

	/** 
	 * Singleton interface for this subsystem 
	 * @return the only instance of this subsystem
	 */
	static FSocketSubsystemMac* Create();

	/**
	 * Performs Mac specific socket clean up
	 */
	static void Destroy();

	/**
	 * Allows a subsystem subclass to create a FSocketBSD sub class 
	 */
	virtual class FSocketBSD* InternalBSDSocketFactory(SOCKET Socket, ESocketType SocketType, const FString& SocketDescription, const FName& SocketProtocol);

public:

	FSocketSubsystemMac() 
		: bTriedToInit(false)
	{
	}

	virtual ~FSocketSubsystemMac()
	{
	}

	/**
	 * Does Mac platform initialization of the sockets library
	 *
	 * @param Error a string that is filled with error information
	 *
	 * @return true if initialized ok, false otherwise
	 */
	virtual bool Init(FString& Error) override;

	/**
	 * Performs platform specific socket clean up
	 */
	virtual void Shutdown() override;

	/**
	 * @return Whether the machine has a properly configured network device or not
	 */
	virtual bool HasNetworkDevice() override;

	virtual bool GetLocalAdapterAddresses(TArray<TSharedPtr<FInternetAddr>>& OutAddresses) override;
};

/**
 * Mac specific socket implementation
 */
class FSocketMac : public FSocketBSD
{
public:

	/**
	 * Assigns a BSD socket to this object
	 *
	 * @param InSocket the socket to assign to this object
	 * @param InSocketType the type of socket that was created
	 * @param InSocketDescription the debug description of the socket
	 */
	FSocketMac(SOCKET InSocket, ESocketType InSocketType, const FString& InSocketDescription, const FName& InSocketProtocol, ISocketSubsystem * InSubsystem) 
	: FSocketBSD(InSocket, InSocketType, InSocketDescription, InSocketProtocol, InSubsystem)
	{
	}

	~FSocketMac()
	{
		FSocketMac::Close();
	}

	virtual bool SetReuseAddr(bool bAllowReuse) override
	{
		int Param = bAllowReuse ? 1 : 0;
		return (setsockopt(Socket,SOL_SOCKET,SO_REUSEADDR,(char*)&Param,sizeof(Param)) == 0) && (setsockopt(Socket,SOL_SOCKET,SO_REUSEPORT,(char*)&Param,sizeof(Param)) == 0);
	}
};