// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultichannelTcpGlobals.h"

DEFINE_LOG_CATEGORY(LogMultichannelTCP);
