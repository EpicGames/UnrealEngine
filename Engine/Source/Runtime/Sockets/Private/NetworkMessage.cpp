// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkMessage.h"
#include "SocketSubsystem.h"
#include "Sockets.h"
#include "MultichannelTcpSocket.h"

bool FSimpleAbstractSocket_FSocket::Receive(uint8 *Results, int32 Size) const
{
	int32 Offset = 0;

	if (Size < 0)
	{
		return false;
	}

	while (Size > 0)
	{
		int32 NumRead = 0;
		Socket->Recv(Results + Offset, Size, NumRead);
		check(NumRead <= Size);
		// make sure we were able to read at least something (and not too much)
		if (NumRead <= 0)
		{
			return false;
		}
		// if we read a partial block, move along
		Offset += NumRead;
		Size -= NumRead;
	}
	return true;
}

bool FSimpleAbstractSocket_FSocket::Send(const uint8 *Buffer, int32 Size) const
{
	while (Size > 0)
	{
		int32 BytesSent = 0;
		if (!Socket->Send(Buffer, Size, BytesSent))
		{
			return false;
		}

		Size -= BytesSent;
		Buffer += BytesSent;
	}

	return true;
}

bool FSimpleAbstractSocket_FMultichannelTCPSocket::Receive(uint8 *Results, int32 Size) const
{
	int32 NumRead = Socket->BlockingReceive(Results, Size, ReceiveChannel);
	// make sure we were able to read at least something (and not too much)
	check(NumRead <= Size);
	if (NumRead <= 0)
	{
		return false;
	}
	return true;
}

bool FSimpleAbstractSocket_FMultichannelTCPSocket::Send(const uint8 *Buffer, int32 Size) const
{
	Socket->Send(Buffer, Size, SendChannel);
	return true;
}

bool FNFSMessageHeader::WrapAndSendPayload(const TArray<uint8>& Payload, const FSimpleAbstractSocket& Socket)
{
	// make a header for the payload
	FNFSMessageHeader Header(Socket, Payload);

	// serialize out the header
	FBufferArchive Ar;
	Ar << Header;

	// append the payload bytes to send it in one network packet
	Ar.Append(Payload);

	// Send it, and make sure it sent it all
	if (!Socket.Send(Ar.GetData(), Ar.Num()))
	{
		UE_LOGF(LogSockets, Error, "Unable to send.");
		return false;
	}
	return true;
}

bool FNFSMessageHeader::ReceivePayload(FArrayReader& OutPayload, const FSimpleAbstractSocket& Socket)
{
	// make room to receive a header
	TArray<uint8> HeaderBytes;
	int32 Size = sizeof(FNFSMessageHeader);
	HeaderBytes.AddZeroed(Size);
		
	if (!Socket.Receive(HeaderBytes.GetData(), Size))
	{
		UE_LOGF(LogSockets, Error, "Unable to read full network header");
		return false;
	}

	// parse it as a header (doing any byte swapping as needed)
	FMemoryReader Reader(HeaderBytes);
	FNFSMessageHeader Header(Socket);
	Reader << Header;

	// make sure it's valid
	if (Header.Magic != Socket.GetMagic())
	{
		UE_LOGF(LogSockets, Error, "Bad network header magic");
		return false;
	}
	if (!Header.PayloadSize)
	{
		UE_LOGF(LogSockets, Error, "Empty payload");
		return false;
	}

	// was the header byteswapped? If so, make the archive byteswapped
	OutPayload.SetByteSwapping(Reader.ForceByteSwapping());

	// we are going to append to the payload, so note how much data is in it now
	int32 PayloadOffset = OutPayload.AddUninitialized(Header.PayloadSize);

	// put the read head at the start of the new data
	OutPayload.Seek(PayloadOffset);

	// receive the payload
	if (!Socket.Receive(OutPayload.GetData() + PayloadOffset, Header.PayloadSize))
	{
		UE_LOGF(LogSockets, Error, "Unable to read full payload");
		return false;
	}

	// make sure it's valid
	uint32 ActualPayloadCrc = FCrc::MemCrc_DEPRECATED(OutPayload.GetData() + PayloadOffset, Header.PayloadSize);
	if (Header.PayloadCrc != ActualPayloadCrc)
	{
		UE_LOGF(LogSockets, Error, "Payload Crc failure.");
		return false;
	}

	// success!
	return true;
}

bool FNFSMessageHeader::SendPayloadAndReceiveResponse(const TArray<uint8>& Payload, FArrayReader& Response, const FSimpleAbstractSocket& Socket)
{

	// send payload
	if (WrapAndSendPayload(Payload, Socket) == false)
	{
		return false;
	}

	// wait for response and get it
	return ReceivePayload(Response, Socket);
}
