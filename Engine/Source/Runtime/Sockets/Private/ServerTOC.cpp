// Copyright Epic Games, Inc. All Rights Reserved.
#include "ServerTOC.h"
#include "Misc/Paths.h"

FServerTOC::~FServerTOC()
{
	for (TMap<FString, FDirectory*>::TIterator DirIt(Directories); DirIt; ++DirIt)
	{
		delete DirIt.Value();
	}
	Directories.Empty();
}

void FServerTOC::AddFileOrDirectory(const FString& Filename, const FDateTime& Timestamp)
{
	// File name
	const FString Path = FPaths::GetPath(Filename);
	FDirectory* ServerPathDirectory = FindDirectory(Path);
	if (ServerPathDirectory == NULL)
	{
		ServerPathDirectory = new FDirectory();
		Directories.Add(Path, ServerPathDirectory);
	}
	// Directories have timestamp = 0, in this case try to add the directory too (GetPath returns the parent directory)
	if (Timestamp == 0)
	{			
		FDirectory* ServerFilenameDirectory = FindDirectory(Filename);
		if (ServerFilenameDirectory == NULL)
		{
			Directories.Add(Filename, new FDirectory());
		}
	}
	// Add file or directory to this directory.
	ServerPathDirectory->Add(Filename, Timestamp);
}

int32 FServerTOC::RemoveFileOrDirectory(const FString& Filename )
{
	const FString Path = FPaths::GetPath( Filename );
	FDirectory* ServerPathDirectory = FindDirectory(Path);
	if ( ServerPathDirectory != NULL )
	{
		return ServerPathDirectory->Remove(Filename);
	}
	return 0;
}


const FDateTime* FServerTOC::FindFile(const FString& Filename) const
{
	const FDateTime* Result = NULL;
	// Find a directory first.
	const FString Path = FPaths::GetPath(Filename);
	const FDirectory*const* FoundDirectory = Directories.Find(Path);
	if (FoundDirectory != NULL)
	{
		// Find a file inside this directory.
		const FDirectory* Directory = *FoundDirectory;
		Result = Directory->Find(Filename);		
	}
	return Result;
}

FServerTOC::FDirectory* FServerTOC::FindDirectory(const FString& Directory) 
{
	FDirectory** FoundDirectory = Directories.Find(Directory);
	if (FoundDirectory != NULL)
	{
		return *FoundDirectory;
	}
	else
	{
		return NULL;
	}
}

const FServerTOC::FDirectory* FServerTOC::FindDirectory(const FString& Directory) const
{
	const FDirectory*const* FoundDirectory = Directories.Find(Directory);
	if (FoundDirectory != NULL)
	{
		return *FoundDirectory;
	}
	else
	{
		return NULL;
	}
}	
