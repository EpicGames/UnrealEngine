// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocketSubsystem.h"
#include "Misc/CommandLine.h"
#include "Misc/ScopeLock.h"
#include "Misc/CoreMisc.h"
#include "SocketSubsystemModule.h"
#include "Modules/ModuleManager.h"
#include "SocketTypes.h"
#include "IPAddress.h"
#include "IPAddressAsyncResolve.h"
#include "Sockets.h"
#include "Templates/UniquePtr.h"

DEFINE_LOG_CATEGORY(LogSockets);

/** Values for the Protocol Typenames */
namespace FNetworkProtocolTypes
{
	const FLazyName IPv4(TEXT("IPv4"));
	const FLazyName IPv6(TEXT("IPv6"));
}

ISocketSubsystem::ISocketSubsystem() = default;
ISocketSubsystem::~ISocketSubsystem() = default;

FUniqueSocket ISocketSubsystem::CreateUniqueSocket(const FName& SocketType, const FString& SocketDescription, bool bForceUDP)
{
	return FUniqueSocket(CreateSocket(SocketType, SocketDescription, bForceUDP), FSocketDeleter(this));
}

FUniqueSocket ISocketSubsystem::CreateUniqueSocket(const FName& SocketType, const FString& SocketDescription, const FName& ProtocolName)
{
	return FUniqueSocket(CreateSocket(SocketType, SocketDescription, ProtocolName), FSocketDeleter(this));
}

/**
 * Shutdown all registered subsystems
 */
void ISocketSubsystem::ShutdownAllSystems()
{
	if (IsInGameThread() && FModuleManager::Get().IsModuleLoaded(TEXT("Sockets")))
	{
		// Unloading the Sockets module will call FSocketSubsystemModule::ShutdownSocketSubsystem()
		const bool bIsShutdown = true;
		FModuleManager::Get().UnloadModule(TEXT("Sockets"), bIsShutdown);
	}
}

//////////////////////////////////
// ISocketSubsystem
/////////////////////////////////

ISocketSubsystem* ISocketSubsystem::Get(const FName& SubsystemName)
{
	// wrap the platform file with a logger
	//	static TUniquePtr<FLoggedPlatformFile> AutoDestroyLog;
	//	AutoDestroyLog = MakeUnique<FLoggedPlatformFile>(*CurrentPlatformFile);

	struct FStatic
	{
		FSocketSubsystemModule& SSSModule;

		FStatic()
			: SSSModule( FModuleManager::LoadModuleChecked<FSocketSubsystemModule>("Sockets") )
		{}

		~FStatic()
		{
			ISocketSubsystem::ShutdownAllSystems();
		}
	};

	static FStatic StaticSockets;
	return StaticSockets.SSSModule.GetSocketSubsystem(SubsystemName); 
}

int32 ISocketSubsystem::BindNextPort(FSocket* Socket, FInternetAddr& Addr, int32 PortCount, int32 PortIncrement)
{
	// go until we reach the limit (or we succeed)
	for (int32 Index = 0; Index < PortCount; Index++)
	{
		// try to bind to the current port
		if (Socket->Bind(Addr) == true)
		{
			// if it succeeded, return the port
			if (Addr.GetPort() != 0)
			{
				return Addr.GetPort();
			}
			else
			{
				return Socket->GetPortNo();
			}
		}
		// if the address had no port, we are done
		if( Addr.GetPort() == 0 )
		{
			break;
		}

		// increment to the next port, and loop!
		Addr.SetPort(Addr.GetPort() + PortIncrement);
	}

	return 0;
}

/** Async task support for GetAddressInfo */
class FGetAddressInfoTask : public FNonAbandonableTask
{
public:
	friend class FAutoDeleteAsyncTask<FGetAddressInfoTask>;

	FGetAddressInfoTask(class ISocketSubsystem* InSocketSubsystem, const FString& InQueryHost,
		const FString& InQueryService, EAddressInfoFlags InQueryFlags, const FName& InQueryProtocol,
		ESocketType InQuerySocketType, FAsyncGetAddressInfoCallback InCallbackFunction) :

		SocketSubsystem(InSocketSubsystem),
		QueryHost(InQueryHost),
		QueryService(InQueryService),
		QueryFlags(InQueryFlags),
		QueryProtocol(InQueryProtocol),
		QuerySocketType(InQuerySocketType),
		CallbackFunction(InCallbackFunction)
	{

	}

	void DoWork()
	{
		CallbackFunction(SocketSubsystem->GetAddressInfo(*QueryHost, *QueryService, QueryFlags, QueryProtocol, QuerySocketType));
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FGetAddressInfoTask, STATGROUP_ThreadPoolAsyncTasks);
	}

private:
	ISocketSubsystem* SocketSubsystem;
	const FString QueryHost;
	const FString QueryService;
	EAddressInfoFlags QueryFlags;
	const FName QueryProtocol;
	ESocketType QuerySocketType;
	FAsyncGetAddressInfoCallback CallbackFunction;
};

void ISocketSubsystem::GetAddressInfoAsync(FAsyncGetAddressInfoCallback Callback, const TCHAR* HostName, const TCHAR* ServiceName,
	EAddressInfoFlags QueryFlags, const FName ProtocolTypeName, ESocketType SocketType)
{
	(new FAutoDeleteAsyncTask<FGetAddressInfoTask>(this, HostName, ServiceName, QueryFlags, ProtocolTypeName, SocketType, Callback))->StartBackgroundTask();
}

TArray<TSharedRef<FInternetAddr>> ISocketSubsystem::GetLocalBindAddresses()
{
	TArray<TSharedRef<FInternetAddr>> BindingAddresses;

	// Multihome addresses should be the only thing in the array if it exists.
	TSharedRef<FInternetAddr> MultihomeAddr = CreateInternetAddr();
	if (GetMultihomeAddress(MultihomeAddr))
	{
		BindingAddresses.Add(MultihomeAddr);
		return BindingAddresses;
	}

	// Next, we want to grab the bindable addresses, which we ask GAI for.
	// Because of how GAI works, we need to push in a service flag otherwise the query will fail.
	//
	// Since these addresses will be initialized to 0 anyways due to the nature of C++, set our service to be the number 0, which will
	// cause no issues in terms of functionality.
	FAddressInfoResult BindableAddresses = GetAddressInfo(nullptr, TEXT("0"),
		EAddressInfoFlags::AllResultsWithMapping | EAddressInfoFlags::BindableAddress | EAddressInfoFlags::OnlyUsableAddresses, NAME_None);

	if (BindableAddresses.ReturnCode == SE_NO_ERROR)
	{
		// Push in all the bindable addresses.
		for (const FAddressInfoResultData& BindAddresses : BindableAddresses.Results)
		{
			// GetAddressInfo can return both TCP and UDP bindings for the same address - which is redundant when returning only addresses
			if (!BindingAddresses.ContainsByPredicate([&](const TSharedRef<FInternetAddr>& A) { return *A == *BindAddresses.Address; }))
			{
				BindingAddresses.Add(BindAddresses.Address);
			}
		}
	}

	return BindingAddresses;
}

bool ISocketSubsystem::GetLocalAdapterAddresses(TArray<TSharedPtr<FInternetAddr>>& OutAddresses)
{
	FString HostName;

	UE_LOGF(LogSockets, Warning, "Falling back to generic GetLocalAdapterAddresses implementation. Consider implementing a Platform specific version.");

	// Attempt to get a hostname so that we can look it up in order to get an idea of adapters that we might have 
	// (or the addresses that are tied to us). This is a fallback implementation for platforms that do not have this implemented.
	// Platforms are encouraged to implement this themselves.
	if (GetHostName(HostName))
	{
		// We want usable addresses, and if possible get every protocol and map it. Some platforms will do this query twice to make sure
		// we have all protocols. Others will only use the protocol specified.
		FAddressInfoResult GAIRequest = GetAddressInfo(*HostName, nullptr,
			EAddressInfoFlags::AllResultsWithMapping | EAddressInfoFlags::OnlyUsableAddresses,
			NAME_None);

		// Start packing the addresses we got to the results.
		if (GAIRequest.ReturnCode == SE_NO_ERROR)
		{
			// Push all results into the output array.
			for (const auto& AddressResult : GAIRequest.Results)
			{
				OutAddresses.Add(AddressResult.Address);
			}
		}

		return true;
	}

	return false;
}

FResolveInfo* ISocketSubsystem::GetHostByName(const ANSICHAR* HostName)
{
	FResolveInfo* Result = NULL;
	TSharedPtr<FInternetAddr> Addr;
	// See if we have it cached or not
	if (GetHostByNameFromCache(HostName, Addr))
	{
		Result = CreateResolveInfoCached(Addr);
	}
	else
	{
		// Create an async resolve info
		FResolveInfoAsync* AsyncResolve = new FResolveInfoAsync(HostName);
		AsyncResolve->StartAsyncTask();
		Result = AsyncResolve;
	}
	return Result;
}

TSharedRef<FInternetAddr> ISocketSubsystem::GetLocalHostAddr(FOutputDevice& Out, bool& bCanBindAll)
{
	TSharedRef<FInternetAddr> HostAddr = CreateInternetAddr();
	bCanBindAll = false;

	if (!GetMultihomeAddress(HostAddr))
	{
		bCanBindAll = true;

		TArray<TSharedPtr<FInternetAddr>> AdapterAddresses;
		if (!GetLocalAdapterAddresses(AdapterAddresses) || (AdapterAddresses.Num() == 0))
		{
			Out.Logf(TEXT("Could not fetch the local adapter addresses"));
			HostAddr->SetAnyAddress();
		}
		else
		{
			if (AdapterAddresses.Num() > 0)
			{
				HostAddr = AdapterAddresses[0]->Clone();
			}
		}
	}

	return HostAddr;
}

TSharedRef<FInternetAddr> ISocketSubsystem::GetLocalBindAddr(FOutputDevice& Out)
{
	TArray<TSharedRef<FInternetAddr>> BindingAddresses = GetLocalBindAddresses();
	if (BindingAddresses.Num() == 0)
	{
		TSharedRef<FInternetAddr> AnyAddress = CreateInternetAddr();
		AnyAddress->SetAnyAddress();
		return AnyAddress;
	}

	return BindingAddresses[0];
}

bool ISocketSubsystem::GetMultihomeAddress(TSharedRef<FInternetAddr>& Addr)
{
	TCHAR Home[256] = {};
	if (FParse::Value(FCommandLine::Get(), TEXT("MULTIHOME="), Home, UE_ARRAY_COUNT(Home)))
	{
		TSharedPtr<FInternetAddr> MultiHomeQuery = GetAddressFromString(Home);
		if (Home == NULL || !MultiHomeQuery.IsValid())
		{
			UE_LOGF(LogSockets, Log, "Invalid multihome IP address %ls", Home);
			return false;
		}
		else
		{
			Addr = MultiHomeQuery.ToSharedRef();
		}

		return true;
	}
	return false;
}

bool ISocketSubsystem::GetHostByNameFromCache(const ANSICHAR* HostName, TSharedPtr<FInternetAddr>& Addr)
{
	// Lock for thread safety
	FScopeLock sl(&HostNameCacheSync);
	// Now search for the entry
	TSharedPtr<FInternetAddr>* FoundAddr = HostNameCache.Find(FString(HostName));
	if (FoundAddr)
	{
		Addr = *FoundAddr;
	}
	return FoundAddr != NULL;
}

void ISocketSubsystem::AddHostNameToCache(const ANSICHAR* HostName, TSharedPtr<FInternetAddr> Addr)
{
	// Lock for thread safety
	FScopeLock sl(&HostNameCacheSync);
	HostNameCache.Add(FString(HostName), Addr);
}

void ISocketSubsystem::RemoveHostNameFromCache(const ANSICHAR* HostName)
{
	// Lock for thread safety
	FScopeLock sl(&HostNameCacheSync);
	HostNameCache.Remove(FString(HostName));
}

ESocketProtocolFamily ISocketSubsystem::GetProtocolFamilyFromName(const FName& InProtocolName) const
{
	if (InProtocolName == FNetworkProtocolTypes::IPv6)
	{
		return ESocketProtocolFamily::IPv6;
	}
	else if (InProtocolName == FNetworkProtocolTypes::IPv4)
	{
		return ESocketProtocolFamily::IPv4;
	}

	return ESocketProtocolFamily::None;
}

FName ISocketSubsystem::GetProtocolNameFromFamily(ESocketProtocolFamily InProtocolFamily) const
{
	switch (InProtocolFamily)
	{
	default:
		return NAME_None;
	case ESocketProtocolFamily::IPv4:
		return FNetworkProtocolTypes::IPv4;
	case ESocketProtocolFamily::IPv6:
		return FNetworkProtocolTypes::IPv6;
	}
}

FResolveInfoCached* ISocketSubsystem::CreateResolveInfoCached(TSharedPtr<FInternetAddr> Addr) const
{
	return new FResolveInfoCached(*Addr);
}

/**
 * Returns a human readable string from an error code
 *
 * @param Code the error code to check
 */
const TCHAR* ISocketSubsystem::GetSocketError(ESocketErrors Code)
{
	if (Code == SE_GET_LAST_ERROR_CODE)
	{
		Code = GetLastErrorCode();
	}
	switch (Code)
	{
		case SE_NO_ERROR: return TEXT("SE_NO_ERROR");
		case SE_EINTR: return TEXT("SE_EINTR");
		case SE_EBADF: return TEXT("SE_EBADF");
		case SE_EACCES: return TEXT("SE_EACCES");
		case SE_EFAULT: return TEXT("SE_EFAULT");
		case SE_EINVAL: return TEXT("SE_EINVAL");
		case SE_EMFILE: return TEXT("SE_EMFILE");
		case SE_EWOULDBLOCK: return TEXT("SE_EWOULDBLOCK");
		case SE_EINPROGRESS: return TEXT("SE_EINPROGRESS");
		case SE_EALREADY: return TEXT("SE_EALREADY");
		case SE_ENOTSOCK: return TEXT("SE_ENOTSOCK");
		case SE_EDESTADDRREQ: return TEXT("SE_EDESTADDRREQ");
		case SE_EMSGSIZE: return TEXT("SE_EMSGSIZE");
		case SE_EPROTOTYPE: return TEXT("SE_EPROTOTYPE");
		case SE_ENOPROTOOPT: return TEXT("SE_ENOPROTOOPT");
		case SE_EPROTONOSUPPORT: return TEXT("SE_EPROTONOSUPPORT");
		case SE_ESOCKTNOSUPPORT: return TEXT("SE_ESOCKTNOSUPPORT");
		case SE_EOPNOTSUPP: return TEXT("SE_EOPNOTSUPP");
		case SE_EPFNOSUPPORT: return TEXT("SE_EPFNOSUPPORT");
		case SE_EAFNOSUPPORT: return TEXT("SE_EAFNOSUPPORT");
		case SE_EADDRINUSE: return TEXT("SE_EADDRINUSE");
		case SE_EADDRNOTAVAIL: return TEXT("SE_EADDRNOTAVAIL");
		case SE_ENETDOWN: return TEXT("SE_ENETDOWN");
		case SE_ENETUNREACH: return TEXT("SE_ENETUNREACH");
		case SE_ENETRESET: return TEXT("SE_ENETRESET");
		case SE_ECONNABORTED: return TEXT("SE_ECONNABORTED");
		case SE_ECONNRESET: return TEXT("SE_ECONNRESET");
		case SE_ENOBUFS: return TEXT("SE_ENOBUFS");
		case SE_EISCONN: return TEXT("SE_EISCONN");
		case SE_ENOTCONN: return TEXT("SE_ENOTCONN");
		case SE_ESHUTDOWN: return TEXT("SE_ESHUTDOWN");
		case SE_ETOOMANYREFS: return TEXT("SE_ETOOMANYREFS");
		case SE_ETIMEDOUT: return TEXT("SE_ETIMEDOUT");
		case SE_ECONNREFUSED: return TEXT("SE_ECONNREFUSED");
		case SE_ELOOP: return TEXT("SE_ELOOP");
		case SE_ENAMETOOLONG: return TEXT("SE_ENAMETOOLONG");
		case SE_EHOSTDOWN: return TEXT("SE_EHOSTDOWN");
		case SE_EHOSTUNREACH: return TEXT("SE_EHOSTUNREACH");
		case SE_ENOTEMPTY: return TEXT("SE_ENOTEMPTY");
		case SE_EPROCLIM: return TEXT("SE_EPROCLIM");
		case SE_EUSERS: return TEXT("SE_EUSERS");
		case SE_EDQUOT: return TEXT("SE_EDQUOT");
		case SE_ESTALE: return TEXT("SE_ESTALE");
		case SE_EREMOTE: return TEXT("SE_EREMOTE");
		case SE_EDISCON: return TEXT("SE_EDISCON");
		case SE_SYSNOTREADY: return TEXT("SE_SYSNOTREADY");
		case SE_VERNOTSUPPORTED: return TEXT("SE_VERNOTSUPPORTED");
		case SE_NOTINITIALISED: return TEXT("SE_NOTINITIALISED");
		case SE_HOST_NOT_FOUND: return TEXT("SE_HOST_NOT_FOUND");
		case SE_TRY_AGAIN: return TEXT("SE_TRY_AGAIN");
		case SE_NO_RECOVERY: return TEXT("SE_NO_RECOVERY");
		case SE_NO_DATA: return TEXT("SE_NO_DATA");
		case SE_UDP_ERR_PORT_UNREACH: return TEXT("SE_UDP_ERR_PORT_UNREACH");
		case SE_ADDRFAMILY: return TEXT("SE_ADDRFAMILY");
		case SE_SYSTEM: return TEXT("SE_SYSTEM");
		case SE_NODEV: return TEXT("SE_NODEV");
		default: return TEXT("Unknown Error");
	};
}

TUniquePtr<FRecvMulti> ISocketSubsystem::CreateRecvMulti(int32 MaxNumPackets, int32 MaxPacketSize,
															ERecvMultiFlags Flags/*=ERecvMultiFlags::None*/)
{
	UE_LOGF(LogSockets, Warning, "RecvMulti is not supported by current socket subsystem.");

	return nullptr;
}

bool ISocketSubsystem::IsSocketRecvMultiSupported() const
{
	return false;
}

double ISocketSubsystem::TranslatePacketTimestamp(const FPacketTimestamp& Timestamp,
													ETimestampTranslation Translation/*=ETimestampTranslation::LocalTimestamp*/)
{
	UE_LOGF(LogSockets, Warning, "TranslatePacketTimestamp is not supported by current socket subsystem.");

	return 0.0;
}

bool ISocketSubsystem::IsRecvFromWithPktInfoSupported() const
{
	return false;
}

void ISocketSubsystem::QuerySocketStateMany(TArrayView<FSocket*> Sockets, TArray<ESocketStateFlags>& InOutSocketStates, FTimespan WaitTime) const
{
	check(!IsSocketWaitSupported());
	
	InOutSocketStates.SetNum(Sockets.Num());

	// clear the returned states
	for (int i = 0; i < InOutSocketStates.Num(); i++)
	{
		InOutSocketStates[i] = (ESocketStateFlags)0;
	}
}

//////////////////////////////////
// SocketSubsystem Testing
/////////////////////////////////
#if WITH_DEV_AUTOMATION_TESTS && !UE_BUILD_SHIPPING

static void DebugPrintGaiResults(ISocketSubsystem* SocketSub, const FAddressInfoResult& GAIResult)
{
	UE_LOGF(LogSockets, Log, "Got %d GAI Results for hostname %ls. Error Code: %ls [%d]", GAIResult.Results.Num(), *GAIResult.QueryHostName,
		SocketSub->GetSocketError(GAIResult.ReturnCode), GAIResult.ReturnCode);

	for (int i = 0; i < GAIResult.Results.Num(); ++i)
	{
		UE_LOGF(LogSockets, Log, "Result #%d Address: %ls Type: %ls", i, *GAIResult.Results[i].Address->ToString(false),
			*GAIResult.Results[i].Address->GetProtocolType().ToString());
	}
}

static void RunGAIQuery(const FString& HostStr)
{
	if (HostStr.IsEmpty())
	{
		UE_LOGF(LogSockets, Warning, "SOCKETSUB GAI requires an input string to test with.");
		return;
	}

	ISocketSubsystem* SocketSub = ISocketSubsystem::Get();
	if (SocketSub)
	{
		FAddressInfoResult GAIResult = SocketSub->GetAddressInfo(*HostStr, nullptr, EAddressInfoFlags::AllResultsWithMapping | EAddressInfoFlags::OnlyUsableAddresses, NAME_None);
		if (GAIResult.Results.Num() > 0)
		{
			DebugPrintGaiResults(SocketSub, GAIResult);
		}
		else
		{
			UE_LOGF(LogSockets, Warning, "Did not get results!");
		}
		return;
	}

	UE_LOGF(LogSockets, Warning, "Failed to get socket subsystem!");
}

static void RunAsyncGAIQuery(const FString& HostStr)
{
	if (HostStr.IsEmpty())
	{
		UE_LOGF(LogSockets, Warning, "SOCKETSUB ASYNCGAI requires an input string to test with.");
		return;
	}

	ISocketSubsystem* SocketSub = ISocketSubsystem::Get();
	if (SocketSub)
	{
		double AsyncRequestStartTime = FPlatformTime::Seconds();
		FAsyncGetAddressInfoCallback CallbackFunc = [SocketSub, AsyncRequestStartTime](FAddressInfoResult Results)
		{
			UE_LOGF(LogSockets, Log, "Async GAI Request returned after %f seconds, started at %f", FPlatformTime::Seconds() - AsyncRequestStartTime, AsyncRequestStartTime);
			DebugPrintGaiResults(SocketSub, Results);
		};
		SocketSub->GetAddressInfoAsync(CallbackFunc, *HostStr, nullptr, EAddressInfoFlags::Default, NAME_None);
		return;
	}

	UE_LOGF(LogSockets, Warning, "Failed to get socket subsystem!");
}

static void RunAddressSerialize(const FString& InputStr)
{
	if (InputStr.IsEmpty())
	{
		UE_LOGF(LogSockets, Warning, "SOCKETSUB Serialize requires an ip address to test with.");
		return;
	}

	ISocketSubsystem* SocketSub = ISocketSubsystem::Get();
	if (SocketSub)
	{
		TSharedPtr<FInternetAddr> NewAddr = SocketSub->GetAddressFromString(InputStr);
		if (NewAddr.IsValid())
		{
			UE_LOGF(LogSockets, Log, "Result Address: %ls Type: %ls", *NewAddr->ToString(false), *NewAddr->GetProtocolType().ToString());
		}
		else
		{
			UE_LOGF(LogSockets, Warning, "Did not get results!");
		}
		return;
	}

	UE_LOGF(LogSockets, Warning, "Failed to get socket subsystem!");
}

static bool SocketSubsystemCommandHandler(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("SOCKETSUB")))
	{
		if (FParse::Command(&Cmd, TEXT("GAI")))
		{
			RunGAIQuery(FParse::Token(Cmd, true));
		}
		else if (FParse::Command(&Cmd, TEXT("SERIALIZE")))
		{
			RunAddressSerialize(FParse::Token(Cmd, true));
		}
		else if (FParse::Command(&Cmd, TEXT("ASYNCGAI")))
		{
			RunAsyncGAIQuery(FParse::Token(Cmd, true));
		}
		else
		{
			ISocketSubsystem* SocketSub = ISocketSubsystem::Get();
			check(SocketSub);

			if (FParse::Command(&Cmd, TEXT("GETBINDADDRESSES")))
			{
				TArray<TSharedRef<FInternetAddr>> BindAddresses = SocketSub->GetLocalBindAddresses();
				UE_LOGF(LogSockets, Log, "Got Binding Addresses:");
				for (const auto& BindAddress : BindAddresses)
				{
					UE_LOGF(LogSockets, Log, "# Bind Address: %ls", *BindAddress->ToString(false));
				}
			}
			else if (FParse::Command(&Cmd, TEXT("GETADAPTERS")))
			{
				TArray<TSharedPtr<FInternetAddr>> AdapterAddresses;
				if (SocketSub->GetLocalAdapterAddresses(AdapterAddresses))
				{
					UE_LOGF(LogSockets, Log, "Got Local Adapter Addresses:");
					for (const auto& AdapterAddress : AdapterAddresses)
					{
						UE_LOGF(LogSockets, Log, "# Adapter Address: %ls", *AdapterAddress->ToString(false));
					}
				}
				else
				{
					UE_LOGF(LogSockets, Warning, "Could not get the local adapter addresses!");
				}
			}
			else if (FParse::Command(&Cmd, TEXT("GETLOCALHOST")))
			{
				bool bCanBindAll = false;
				TSharedRef<FInternetAddr> LocalHostAddr = SocketSub->GetLocalHostAddr(Ar, bCanBindAll);
				UE_LOGF(LogSockets, Log, "LocalHostAddr is %ls (this function auto adds multihome) Bind all: %d", *LocalHostAddr->ToString(false), bCanBindAll);
			}
			else if (FParse::Command(&Cmd, TEXT("GETBINDADDR")))
			{
				TSharedRef<FInternetAddr> BindAddr = SocketSub->GetLocalBindAddr(Ar);
				UE_LOGF(LogSockets, Log, "The Local binding addr is %ls", *BindAddr->ToString(false));
			}
		}

		return true;
	}

	return false;
}

FStaticSelfRegisteringExec FSocketSubsystemExecs(SocketSubsystemCommandHandler);
#endif // WITH_DEV_AUTOMATION_TESTS && !UE_BUILD_SHIPPING
