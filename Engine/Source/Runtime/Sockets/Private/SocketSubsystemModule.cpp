// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocketSubsystemModule.h"

#include "Misc/CommandLine.h"
#include "Modules/ModuleManager.h"
#include "SocketSubsystem.h"

IMPLEMENT_MODULE( FSocketSubsystemModule, Sockets );

/** Each platform will implement these functions to construct/destroy socket implementations */
extern FName CreateSocketSubsystem(FSocketSubsystemModule& SocketSubsystemModule);
extern void DestroySocketSubsystem(FSocketSubsystemModule& SocketSubsystemModule);

/** Helper function to turn the friendly subsystem name into the module name */
static inline FName GetSocketModuleName(const FString& SubsystemName)
{
	FName ModuleName;
	FString SocketBaseName("Sockets");

	if (!SubsystemName.StartsWith(SocketBaseName, ESearchCase::CaseSensitive))
	{
		ModuleName = FName(*(SocketBaseName + SubsystemName));
	}
	else
	{
		ModuleName = FName(*SubsystemName);
	}

	return ModuleName;
}

/**
 * Helper function that loads a given platform service module if it isn't already loaded
 *
 * @param SubsystemName Name of the requested platform service to load
 * @return The module interface of the requested platform service, NULL if the service doesn't exist
 */
static IModuleInterface* LoadSubsystemModule(const FString& SubsystemName)
{
#if !UE_BUILD_SHIPPING && !UE_BUILD_SHIPPING_WITH_EDITOR
	// Early out if we are overriding the module load
	bool bAttemptLoadModule = !FParse::Param(FCommandLine::Get(), *FString::Printf(TEXT("no%s"), *SubsystemName));
	if (bAttemptLoadModule)
#endif
	{
		FName ModuleName;
		FModuleManager& ModuleManager = FModuleManager::Get();

		ModuleName = GetSocketModuleName(SubsystemName);
		if (!ModuleManager.IsModuleLoaded(ModuleName))
		{
			// Attempt to load the module
			ModuleManager.LoadModule(ModuleName);
		}

		return ModuleManager.GetModule(ModuleName);
	}

#if !UE_BUILD_SHIPPING && !UE_BUILD_SHIPPING_WITH_EDITOR
	return nullptr;
#endif
}

ISocketSubsystem* FSocketSubsystemModule::GetSocketSubsystem(const FName InSubsystemName)
{
	FName SubsystemName = InSubsystemName;
	if (SubsystemName == NAME_None)
	{
		SubsystemName = DefaultSocketSubsystem;
	}

	ISocketSubsystem** SocketSubsystemFactory = SocketSubsystems.Find(SubsystemName);
	if (SocketSubsystemFactory == nullptr)
	{
		// Attempt to load the requested factory
		IModuleInterface* NewModule = LoadSubsystemModule(SubsystemName.ToString());
		if (NewModule)
		{
			// If the module loaded successfully this should be non-NULL;
			SocketSubsystemFactory = SocketSubsystems.Find(SubsystemName);
		}

		if (SocketSubsystemFactory == nullptr)
		{
			UE_LOGF(LogSockets, Warning, "Unable to load SocketSubsystem module %ls", *InSubsystemName.ToString());
		}
	}

	return (SocketSubsystemFactory == nullptr) ? nullptr : *SocketSubsystemFactory;
}

void FSocketSubsystemModule::RegisterSocketSubsystem(const FName FactoryName, ISocketSubsystem* Factory, bool bMakeDefault)
{
	if (!SocketSubsystems.Contains(FactoryName))
	{
		SocketSubsystems.Add(FactoryName, Factory);
	}

	if (bMakeDefault)
	{
		DefaultSocketSubsystem = FactoryName;
	}
}

void FSocketSubsystemModule::UnregisterSocketSubsystem(const FName FactoryName)
{
	if (SocketSubsystems.Contains(FactoryName))
	{
		SocketSubsystems.Remove(FactoryName);
	}
}

void FSocketSubsystemModule::StartupModule()
{
	FString InterfaceString;

	// Initialize the platform defined socket subsystem first
	DefaultSocketSubsystem = CreateSocketSubsystem( *this );
}

void FSocketSubsystemModule::ShutdownModule()
{
	// Destroy the platform defined socket subsystem first
	DestroySocketSubsystem( *this );

	FModuleManager& ModuleManager = FModuleManager::Get();
	// Unload all the supporting factories
	for (TMap<FName, ISocketSubsystem*>::TIterator It(SocketSubsystems); It; ++It)
	{
		It.Value()->Shutdown();
		// Unloading the module will do proper cleanup
		FName ModuleName = GetSocketModuleName(It.Key().ToString());

		const bool bIsShutdown = true;
		ModuleManager.UnloadModule(ModuleName, bIsShutdown);
	}
}
