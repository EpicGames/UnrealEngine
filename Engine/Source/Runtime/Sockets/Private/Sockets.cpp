// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sockets.h"
#include "SocketSubsystem.h"
//#include "Net/NetworkProfiler.h"


/**
 * FRecvMulti
 */

FRecvMulti::FRecvMulti(ISocketSubsystem* SocketSubsystem, int32 InMaxNumPackets, int32 InMaxPacketSize, ERecvMultiFlags InitFlags)
	: Packets(MakeUnique<FRecvData[]>(InMaxNumPackets))
	, NumPackets(0)
	, MaxNumPackets(InMaxNumPackets)
	, MaxPacketSize(InMaxPacketSize)
{
}

void FRecvMulti::CountBytes(FArchive& Ar) const
{
	Ar.CountBytes(sizeof(*this), sizeof(*this));

	// Packets
	Ar.CountBytes(MaxNumPackets * sizeof(FRecvData), MaxNumPackets * sizeof(FRecvData));
}

//
// FSocket stats implementation
//

FSocket::FSocket() :
	SocketType(SOCKTYPE_Unknown),
	SocketDescription(TEXT("")),
	SocketProtocol(NAME_None)
{ }

FSocket::FSocket(ESocketType InSocketType, const FString& InSocketDescription, const FName& InSocketProtocol) :
	SocketType(InSocketType),
	SocketDescription(InSocketDescription),
	SocketProtocol(InSocketProtocol)
{ }

/** Virtual destructor. */
FSocket::~FSocket() = default;

bool FSocket::SendTo(const uint8* Data, int32 Count, int32& BytesSent, const FInternetAddr& Destination)
{
//	NETWORK_PROFILER(GNetworkProfiler.TrackSocketSendTo(this,Data,BytesSent,Destination));
	UE_LOGF(LogSockets, Verbose, "Socket '%ls' SendTo %i Bytes", *SocketDescription, BytesSent );
	return true;
}


bool FSocket::Send(const uint8* Data, int32 Count, int32& BytesSent)
{
//	NETWORK_PROFILER(GNetworkProfiler.TrackSocketSend(this,Data,BytesSent));
	UE_LOGF(LogSockets, Verbose, "Socket '%ls' Send %i Bytes", *SocketDescription, BytesSent );
	return true;
}


bool FSocket::RecvFrom(uint8* Data, int32 BufferSize, int32& BytesRead, FInternetAddr& Source, ESocketReceiveFlags::Type Flags)
{
	if( BytesRead > 0 )
	{
		UE_LOGF(LogSockets, Verbose, "Socket '%ls' RecvFrom %i Bytes", *SocketDescription, BytesRead );
	}
	return true;
}


bool FSocket::Recv(uint8* Data, int32 BufferSize, int32& BytesRead, ESocketReceiveFlags::Type Flags)
{
	if( BytesRead > 0 )
	{
		UE_LOGF(LogSockets, Verbose, "Socket '%ls' Recv %i Bytes", *SocketDescription, BytesRead );
	}
	return true;
}

bool FSocket::RecvMulti(FRecvMulti& MultiData, ESocketReceiveFlags::Type Flags/*=ESocketReceiveFlags::None*/)
{
	return false;
}

bool FSocket::SetRetrieveTimestamp(bool bRetrieveTimestamp/*=true*/)
{
	return false;
}

bool FSocket::SetSendSegmentationOffloadSize(uint16 InSegmentPayloadBytes)
{
	return false;
}

bool FSocket::SetIpPktInfo(bool bEnable)
{
	return false;
}

bool FSocket::RecvFromWithPktInfo(uint8* Data, int32 BufferSize, int32& BytesRead, FInternetAddr& Source, FInternetAddr& Destination, ESocketReceiveFlags::Type Flags)
{
	return false;
}
