// Copyright Epic Games, Inc. All Rights Reserved.

#include "Unix/SocketSubsystemUnix.h"
#include "Misc/CommandLine.h"
#include "SocketSubsystemModule.h"
#include "BSDSockets/IPAddressBSD.h"
#include "Unix/SocketsUnix.h"

#include <ifaddrs.h>
#include <net/if.h>

FSocketSubsystemUnix* FSocketSubsystemUnix::SocketSingleton = NULL;

FName CreateSocketSubsystem( FSocketSubsystemModule& SocketSubsystemModule )
{
	FName SubsystemName("UNIX");
	// Create and register our singleton factor with the main online subsystem for easy access
	FSocketSubsystemUnix* SocketSubsystem = FSocketSubsystemUnix::Create();
	FString Error;
	if (SocketSubsystem->Init(Error))
	{
		SocketSubsystemModule.RegisterSocketSubsystem(SubsystemName, SocketSubsystem);
		return SubsystemName;
	}
	else
	{
		FSocketSubsystemUnix::Destroy();
		return NAME_None;
	}
}

void DestroySocketSubsystem( FSocketSubsystemModule& SocketSubsystemModule )
{
	SocketSubsystemModule.UnregisterSocketSubsystem(FName("UNIX"));
	FSocketSubsystemUnix::Destroy();
}

/** 
 * Singleton interface for the Android socket subsystem 
 * @return the only instance of the Android socket subsystem
 */
FSocketSubsystemUnix* FSocketSubsystemUnix::Create()
{
	if (SocketSingleton == NULL)
	{
		SocketSingleton = new FSocketSubsystemUnix();
	}

	return SocketSingleton;
}

/** 
 * Destroy the singleton Android socket subsystem
 */
void FSocketSubsystemUnix::Destroy()
{
	if (SocketSingleton != NULL)
	{
		SocketSingleton->Shutdown();
		delete SocketSingleton;
		SocketSingleton = NULL;
	}
}

/**
 * Does Unix platform initialization of the sockets library
 *
 * @param Error a string that is filled with error information
 *
 * @return TRUE if initialized ok, FALSE otherwise
 */
bool FSocketSubsystemUnix::Init(FString& Error)
{
	return true;
}

/**
 * Performs Android specific socket clean up
 */
void FSocketSubsystemUnix::Shutdown(void)
{
}


/**
 * @return Whether the device has a properly configured network device or not
 */
bool FSocketSubsystemUnix::HasNetworkDevice()
{
	// @TODO: implement
	return true;
}

FSocket* FSocketSubsystemUnix::CreateSocket(const FName& SocketType, const FString& SocketDescription, const FName& ProtocolType)
{
	FSocketBSD* NewSocket = (FSocketBSD*)FSocketSubsystemBSD::CreateSocket(SocketType, SocketDescription, ProtocolType);

	if (NewSocket != nullptr)
	{
		NewSocket->SetIPv6Only(false);
	}
	else
	{
		UE_LOGF(LogSockets, Warning, "Failed to create socket %ls [%ls]", *SocketType.ToString(), *SocketDescription);
	}

	return NewSocket;
}

bool FSocketSubsystemUnix::GetLocalAdapterAddresses(TArray<TSharedPtr<FInternetAddr>>& OutAddresses)
{
	ifaddrs* Interfaces = NULL;
	int InterfaceQueryRet = getifaddrs(&Interfaces);
	UE_LOGF(LogSockets, Verbose, "Querying net interfaces returned: %d", InterfaceQueryRet);
	const bool bDisableIPv6 = CVarDisableIPv6.GetValueOnAnyThread() == 1;
	if (InterfaceQueryRet == 0)
	{
		// Loop through linked list of interfaces
		for (ifaddrs* Travel = Interfaces; Travel != NULL; Travel = Travel->ifa_next)
		{
			// Skip over empty data sets.
			if (Travel->ifa_addr == NULL)
			{
				continue;
			}

			uint16 AddrFamily = Travel->ifa_addr->sa_family;
			// Find any up and non-loopback addresses
			if ((Travel->ifa_flags & IFF_UP) != 0 &&
				(Travel->ifa_flags & IFF_LOOPBACK) == 0 && 
				(AddrFamily == AF_INET || (!bDisableIPv6 && AddrFamily == AF_INET6)))
			{
				TSharedRef<FInternetAddrBSD> NewAddress = MakeShareable(new FInternetAddrBSD(this));
				NewAddress->SetIp(*((sockaddr_storage*)Travel->ifa_addr));
				uint32 AddressInterface = ntohl(if_nametoindex(Travel->ifa_name));

				NewAddress->SetScopeId(AddressInterface);
				OutAddresses.Add(NewAddress);
				UE_LOGF(LogSockets, Verbose, "Added address %ls on interface %d", *(NewAddress->ToString(false)), AddressInterface);
			}
		}

		freeifaddrs(Interfaces);
	}
	else
	{
		UE_LOGF(LogSockets, Warning, "getifaddrs returned result %d", InterfaceQueryRet);
		return false;
	}

	return (OutAddresses.Num() > 0);
}

FSocketBSD* FSocketSubsystemUnix::InternalBSDSocketFactory(SOCKET Socket, ESocketType SocketType, const FString& SocketDescription, const FName& SocketProtocol)
{
	return new FSocketUnix(Socket, SocketType, SocketDescription, SocketProtocol, this);
}

TUniquePtr<FRecvMulti> FSocketSubsystemUnix::CreateRecvMulti(int32 MaxNumPackets, int32 MaxPacketSize, ERecvMultiFlags Flags)
{
#if PLATFORM_HAS_BSD_SOCKET_FEATURE_RECVMMSG
	return MakeUnique<FUnixRecvMulti>(this, MaxNumPackets, MaxPacketSize, Flags);
#else
	return nullptr;
#endif
}

bool FSocketSubsystemUnix::IsSocketRecvMultiSupported() const
{
#if PLATFORM_HAS_BSD_SOCKET_FEATURE_RECVMMSG
	return true;
#else
	return false;
#endif
}

double FSocketSubsystemUnix::TranslatePacketTimestamp(const FPacketTimestamp& Timestamp, ETimestampTranslation Translation)
{
	double ReturnVal = 0.0;
	const bool bDeltaOnly = Translation == ETimestampTranslation::TimeDelta;

	if (Translation == ETimestampTranslation::LocalTimestamp || bDeltaOnly)
	{
		// Unfortunately, the packet timestamp is platform-specific, using CLOCK_REALTIME in this case,
		// whereas FPlatformTime may select from a variety of incompatible clocks.
		// So, the only safe option is to return the time difference instead.
		struct timespec CurPlatformTime;

		clock_gettime(CLOCK_REALTIME, &CurPlatformTime);

		FTimespan CurPlatformTimespan((CurPlatformTime.tv_sec * ETimespan::TicksPerSecond) +
										(CurPlatformTime.tv_nsec / ETimespan::NanosecondsPerTick));

		ReturnVal = (CurPlatformTimespan - Timestamp.Timestamp).GetTotalSeconds();

		if (!bDeltaOnly)
		{
			ReturnVal = FPlatformTime::Seconds() - ReturnVal;
		}
	}
	else
	{
		UE_LOGF(LogSockets, Warning, "TranslatePacketTimestamp: Unknown ETimestampTranslation type: %i", (uint32)Translation);
	}

	return ReturnVal;
}

