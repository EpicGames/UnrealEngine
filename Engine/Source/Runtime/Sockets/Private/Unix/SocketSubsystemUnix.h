// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BSDSockets/SocketSubsystemBSD.h"
#include "SocketSubsystemPackage.h"

class Error;
class FInternetAddr;

/**
 * Unix specific socket subsystem implementation
 */
class FSocketSubsystemUnix : public FSocketSubsystemBSD
{
protected:

	/** Single instantiation of this subsystem */
	static FSocketSubsystemUnix* SocketSingleton;

	/** Whether Init() has been called before or not */
	bool bTriedToInit;

PACKAGE_SCOPE:

	/** 
	 * Singleton interface for this subsystem 
	 * @return the only instance of this subsystem
	 */
	static FSocketSubsystemUnix* Create();

	/**
	 * Performs Unix specific socket clean up
	 */
	static void Destroy();

public:

	FSocketSubsystemUnix() 
		: bTriedToInit(false)
	{
	}

	virtual ~FSocketSubsystemUnix() = default;

	// ISocketSubsystem
	virtual bool Init(FString& Error) override;
	virtual void Shutdown() override;
	virtual bool HasNetworkDevice() override;
	virtual class FSocket* CreateSocket(const FName& SocketType, const FString& SocketDescription, const FName& ProtocolType) override;
	virtual bool GetLocalAdapterAddresses(TArray<TSharedPtr<FInternetAddr> >& OutAddresses) override;
	virtual class FSocketBSD* InternalBSDSocketFactory( SOCKET Socket, ESocketType SocketType, const FString& SocketDescription, const FName& SocketProtocol) override;
	virtual TUniquePtr<FRecvMulti> CreateRecvMulti(int32 MaxNumPackets, int32 MaxPacketSize, ERecvMultiFlags Flags) override;
	virtual bool IsSocketRecvMultiSupported() const override;
	virtual double TranslatePacketTimestamp(const FPacketTimestamp& Timestamp, ETimestampTranslation Translation) override;
};
