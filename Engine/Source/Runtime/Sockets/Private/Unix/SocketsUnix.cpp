// Copyright Epic Games, Inc. All Rights Reserved


#include "SocketsUnix.h"
#include "BSDSockets/IPAddressBSD.h"


// @todo: Add timestamp support for normal Recv/RecvFrom (not essential, there is no API for this yet)


#if PLATFORM_HAS_BSD_SOCKET_FEATURE_RECVMMSG
constexpr const int32 TimestampControlMsgSize		= CMSG_SPACE(sizeof(timeval));
constexpr const int32 TimestampControlReturnSize	= CMSG_LEN(sizeof(timeval));


/**
 * FUnixRecvMulti
 */

FUnixRecvMulti::FUnixRecvMulti(ISocketSubsystem* SocketSubsystem, int32 InMaxNumPackets, int32 InMaxPacketSize,
								ERecvMultiFlags InitFlags)
	: FRecvMulti(SocketSubsystem, InMaxNumPackets, InMaxPacketSize, InitFlags)
	, Headers(MakeUnique<mmsghdr[]>(MaxNumPackets))
	, BufferMaps(MakeUnique<iovec[]>(MaxNumPackets))
	, DataBuffer(MakeUnique<uint8[]>(MaxNumPackets * MaxPacketSize))
{
	bool bRetrieveTimestamps = EnumHasAnyFlags(InitFlags, ERecvMultiFlags::RetrieveTimestamps);
	RawTimestampData = (bRetrieveTimestamps ? MakeUnique<uint8[]>(TimestampControlMsgSize * MaxNumPackets) : nullptr);

	for (int32 i=0; i<MaxNumPackets; i++)
	{
		mmsghdr& CurHeader = Headers[i];
		msghdr& CurInnerHeader = CurHeader.msg_hdr;
		iovec& CurBufferMap = BufferMaps[i];
		FRecvMulti::FRecvData& CurPacket = Packets[i];

		CurPacket.Data = &DataBuffer[MaxPacketSize*i];
		CurPacket.Source = SocketSubsystem->CreateInternetAddr();
		CurPacket.BytesReadPtr = &CurHeader.msg_len;


		FInternetAddrBSD* CurBSDAddr = (FInternetAddrBSD*)CurPacket.Source.Get();

		CurInnerHeader.msg_name = CurBSDAddr->GetRawAddr();
		CurInnerHeader.msg_namelen = CurBSDAddr->GetStorageSize();
		CurInnerHeader.msg_iov = &CurBufferMap;
		CurInnerHeader.msg_iovlen = 1;
		CurInnerHeader.msg_control = (bRetrieveTimestamps ? &RawTimestampData[i * TimestampControlMsgSize] : nullptr);;
		CurInnerHeader.msg_controllen = (bRetrieveTimestamps ? TimestampControlMsgSize : 0);
		CurInnerHeader.msg_flags = 0;

		CurBufferMap.iov_base = (void*)CurPacket.Data;
		CurBufferMap.iov_len = MaxPacketSize;
	}
}

bool FUnixRecvMulti::GetPacketTimestamp(int32 PacketIdx, FPacketTimestamp& OutTimestamp) const
{
	bool bSuccess = false;

	if (RawTimestampData.IsValid())
	{
		msghdr& CurInnerHeader = Headers[PacketIdx].msg_hdr;
		const cmsghdr* TimestampMsg = CMSG_FIRSTHDR(&CurInnerHeader);

		if (TimestampMsg != nullptr && TimestampMsg->cmsg_level == SOL_SOCKET &&
			TimestampMsg->cmsg_type == SCM_TIMESTAMP && TimestampMsg->cmsg_len == TimestampControlReturnSize)
		{
			timeval* CurTimestampData = (timeval*)CMSG_DATA(TimestampMsg);

			if (CurTimestampData != nullptr)
			{
				OutTimestamp.Timestamp = FTimespan((CurTimestampData->tv_sec * ETimespan::TicksPerSecond) +
											(CurTimestampData->tv_usec * ETimespan::TicksPerMicrosecond));

				bSuccess = true;
			}
		}
	}

	return bSuccess;
}

void FUnixRecvMulti::CountBytes(FArchive& Ar) const
{
	FRecvMulti::CountBytes(Ar);

	int32 CurSize = sizeof(*this) - sizeof(FRecvMulti);

	Ar.CountBytes(CurSize, CurSize);

	// Headers
	CurSize = sizeof(mmsghdr) * MaxNumPackets;

	Ar.CountBytes(CurSize, CurSize);

	// RawTimestampData
	CurSize = (RawTimestampData.IsValid() ? (TimestampControlMsgSize * MaxNumPackets) : 0);

	Ar.CountBytes(CurSize, CurSize);

	// BufferMaps
	CurSize = sizeof(iovec) * MaxNumPackets;

	Ar.CountBytes(CurSize, CurSize);

	// FInternetAddrBSD
	CurSize = sizeof(FInternetAddrBSD) * MaxNumPackets;

	Ar.CountBytes(CurSize, CurSize);

	// DataBuffer
	Ar.CountBytes(MaxNumPackets, MaxNumPackets);
}
#endif


/**
 * FSocketUnix
 */

// NOTE: Does not support TCP at the moment.
bool FSocketUnix::RecvMulti(FRecvMulti& MultiData, ESocketReceiveFlags::Type Flags)
{
	bool bSuccess = false;

#if PLATFORM_HAS_BSD_SOCKET_FEATURE_RECVMMSG
	FUnixRecvMulti& UnixMultiData = (FUnixRecvMulti&)MultiData;
	mmsghdr* Headers = UnixMultiData.Headers.Get();
	const int TranslatedFlags = TranslateFlags(Flags);
	int NumPacketsRead = recvmmsg(Socket, Headers, UnixMultiData.MaxNumPackets, TranslatedFlags, nullptr);

	bSuccess = NumPacketsRead > 0;
	UnixMultiData.NumPackets = FMath::Max(0, NumPacketsRead);

	if (bSuccess)
	{
		LastActivityTime = FPlatformTime::Seconds();
	}
#endif

	return bSuccess;
}

bool FSocketUnix::SetRetrieveTimestamp(bool bRetrieveTimestamp)
{
	bool bSuccess = false;

#if PLATFORM_HAS_BSD_SOCKET_FEATURE_TIMESTAMP
	int32 Timestamp = (int)bRetrieveTimestamp;

	bSuccess = setsockopt(Socket, SOL_SOCKET, SO_TIMESTAMP, (char*)&Timestamp, sizeof(Timestamp)) == 0;
#endif

	return bSuccess;
}
