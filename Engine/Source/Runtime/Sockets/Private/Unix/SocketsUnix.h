// Copyright Epic Games, Inc. All Rights Reserved

#pragma once


#include "BSDSockets/SocketsBSD.h"


#if PLATFORM_HAS_BSD_SOCKET_FEATURE_RECVMMSG
/**
 * Implements platform specific data/buffers for RecvMulti in Linux
 */
struct FUnixRecvMulti : public FRecvMulti
{
	friend class FSocketUnix;

protected:
	/** Persistently stores preconfigured mmsghdr struct values, for use with recvmmsg */
	TUniquePtr<mmsghdr[]>	Headers;

	/** Buffer for receiving/storing packet kernel timestamp data, during the call to recvmmsg */
	TUniquePtr<uint8[]>		RawTimestampData;


private:
	/** Maps sections of DataBuffer for receiving packet data, within Headers */
	TUniquePtr<iovec[]>		BufferMaps;

	/** The raw data buffer where all packet data is received/stored. */
	TUniquePtr<uint8[]>		DataBuffer;


public:
	FUnixRecvMulti(ISocketSubsystem* SocketSubsystem, int32 InMaxNumPackets, int32 InMaxPacketSize, ERecvMultiFlags InitFlags);

	virtual bool GetPacketTimestamp(int32 PacketIdx, FPacketTimestamp& OutTimestamp) const override;
	virtual void CountBytes(FArchive& Ar) const override;
};
#endif


/**
 * Unix specific socket implementation - primarily, adds support for recvmmsg
 */
class FSocketUnix : public FSocketBSD
{
public:
	/**
	 * Base constructor
	 */
	FSocketUnix(SOCKET InSocket, ESocketType InSocketType, const FString& InSocketDescription, const FName& InSocketProtocol,
				ISocketSubsystem* InSubsystem)
		: FSocketBSD(InSocket, InSocketType, InSocketDescription, InSocketProtocol, InSubsystem)
	{
	}

	virtual bool RecvMulti(FRecvMulti& MultiData, ESocketReceiveFlags::Type Flags) override;
	virtual bool SetRetrieveTimestamp(bool bRetrieveTimestamp) override;
};
