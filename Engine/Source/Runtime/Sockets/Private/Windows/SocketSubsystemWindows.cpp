// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocketSubsystemWindows.h"
#include "SocketSubsystemModule.h"
#include "Modules/ModuleManager.h"
#include "BSDSockets/IPAddressBSD.h"

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include "Iphlpapi.h"
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"


FSocketSubsystemWindows* FSocketSubsystemWindows::SocketSingleton = nullptr;


FName CreateSocketSubsystem( FSocketSubsystemModule& SocketSubsystemModule )
{
	FName SubsystemName("WINDOWS");

	// Create and register our singleton factory with the main online subsystem for easy access
	FSocketSubsystemWindows* SocketSubsystem = FSocketSubsystemWindows::Create();
	FString Error;

	if (SocketSubsystem->Init(Error))
	{
		SocketSubsystemModule.RegisterSocketSubsystem(SubsystemName, SocketSubsystem);

		return SubsystemName;
	}

	FSocketSubsystemWindows::Destroy();
	
	return NAME_None;
}


void DestroySocketSubsystem( FSocketSubsystemModule& SocketSubsystemModule )
{
	SocketSubsystemModule.UnregisterSocketSubsystem(FName("WINDOWS"));
	FSocketSubsystemWindows::Destroy();
}


/* FSocketSubsystemWindows interface
*****************************************************************************/

FSocketSubsystemWindows* FSocketSubsystemWindows::Create()
{
	if (SocketSingleton == nullptr)
	{
		SocketSingleton = new FSocketSubsystemWindows();
	}

	return SocketSingleton;
}


void FSocketSubsystemWindows::Destroy()
{
	if (SocketSingleton != nullptr)
	{
		SocketSingleton->Shutdown();
		delete SocketSingleton;
		SocketSingleton = nullptr;
	}
}


/* FSocketSubsystemBSD overrides
*****************************************************************************/

FSocket* FSocketSubsystemWindows::CreateSocket(const FName& SocketType, const FString& SocketDescription, const FName& ProtocolType)
{
	FSocketWindows* NewSocket = (FSocketWindows*)FSocketSubsystemBSD::CreateSocket(SocketType, SocketDescription, ProtocolType);

	if (NewSocket != nullptr)
	{
		::SetHandleInformation((HANDLE)NewSocket->GetNativeSocket(), HANDLE_FLAG_INHERIT, 0);
		NewSocket->SetIPv6Only(false);
	}
	else
	{
		UE_LOGF(LogSockets, Warning, "Failed to create socket %ls [%ls]", *SocketType.ToString(), *SocketDescription);
	}

	return NewSocket;
}


bool FSocketSubsystemWindows::Init(FString& Error)
{
	bool bSuccess = false;

	if (bTriedToInit == false)
	{
		bTriedToInit = true;
		WSADATA WSAData;

		// initialize WSA
		int32 Code = WSAStartup(0x0101, &WSAData);

		if (Code == 0)
		{
			bSuccess = true;
			UE_LOGF(LogInit, Log, "%ls: version %i.%i (%i.%i), MaxSocks=%i, MaxUdp=%i",
				GetSocketAPIName(),
				WSAData.wVersion >> 8, WSAData.wVersion & 0xFF,
				WSAData.wHighVersion >> 8, WSAData.wHighVersion & 0xFF,
				WSAData.iMaxSockets, WSAData.iMaxUdpDg);
		}
		else
		{
			Error = FString::Printf(TEXT("WSAStartup failed (%s)"), GetSocketError(TranslateErrorCode(Code)));
		}
	}

	return bSuccess;
}


void FSocketSubsystemWindows::Shutdown(void)
{
	WSACleanup();
}


ESocketErrors FSocketSubsystemWindows::GetLastErrorCode()
{
	return TranslateErrorCode(WSAGetLastError());
}


bool FSocketSubsystemWindows::GetLocalAdapterAddresses(TArray<TSharedPtr<FInternetAddr>>& OutAddresses)
{
	ULONG Flags = GAA_FLAG_INCLUDE_PREFIX | GAA_FLAG_SKIP_MULTICAST | GAA_FLAG_SKIP_ANYCAST | GAA_FLAG_SKIP_DNS_SERVER | GAA_FLAG_SKIP_FRIENDLY_NAME;
	ULONG Result;
	ULONG Size = 0;
	ULONG Family = (PLATFORM_HAS_BSD_IPV6_SOCKETS && CVarDisableIPv6.GetValueOnAnyThread() == 0) ? AF_UNSPEC : AF_INET;

	// determine the required size of the address list buffer
	Result = GetAdaptersAddresses(Family, Flags, NULL, NULL, &Size);

	if (Result != ERROR_BUFFER_OVERFLOW)
	{
		return false;
	}

	PIP_ADAPTER_ADDRESSES AdapterAddresses = (PIP_ADAPTER_ADDRESSES)FMemory::Malloc(Size);

	// get the actual list of adapters
	Result = GetAdaptersAddresses(Family, Flags, NULL, AdapterAddresses, &Size);
	
	if (Result != ERROR_SUCCESS)
	{
		FMemory::Free(AdapterAddresses);
		return false;
	}

	// extract the list of physical addresses from each adapter
	for (PIP_ADAPTER_ADDRESSES AdapterAddress = AdapterAddresses; AdapterAddress != NULL; AdapterAddress = AdapterAddress->Next)
	{
		if ((AdapterAddress->IfType == IF_TYPE_ETHERNET_CSMACD ||
			AdapterAddress->IfType == IF_TYPE_IEEE80211) && 
			AdapterAddress->OperStatus == IfOperStatusUp)
		{
			for (PIP_ADAPTER_UNICAST_ADDRESS UnicastAddress = AdapterAddress->FirstUnicastAddress; UnicastAddress != NULL; UnicastAddress = UnicastAddress->Next)
			{
				if ((UnicastAddress->Flags & IP_ADAPTER_ADDRESS_DNS_ELIGIBLE) != 0)
				{
					const sockaddr_storage* RawAddress = (const sockaddr_storage*)(UnicastAddress->Address.lpSockaddr);
					TSharedRef<FInternetAddrBSD> NewAddress = MakeShareable(new FInternetAddrBSD(this));
					NewAddress->SetIp(*RawAddress);
					NewAddress->SetScopeId((RawAddress->ss_family == AF_INET) ? ntohl(AdapterAddress->IfIndex) : ntohl(AdapterAddress->Ipv6IfIndex));
					UE_LOGF(LogSockets, Verbose, "Adapter address added: %ls", *NewAddress->ToString(false));
					OutAddresses.Add(NewAddress);
				}
			}
		}
	}

	FMemory::Free(AdapterAddresses);
	return true;
}


ESocketErrors FSocketSubsystemWindows::TranslateErrorCode(int32 Code)
{
	// handle the generic -1 error
	if (Code == SOCKET_ERROR)
	{
		return GetLastErrorCode();
	}

	switch (Code)
	{
	case 0: return SE_NO_ERROR;
	case ERROR_INVALID_HANDLE: return SE_ECONNRESET; // invalid socket handle catch
	case WSAEINTR: return SE_EINTR;
	case WSAEBADF: return SE_EBADF;
	case WSAEACCES: return SE_EACCES;
	case WSAEFAULT: return SE_EFAULT;
	case WSAEINVAL: return SE_EINVAL;
	case WSAEMFILE: return SE_EMFILE;
	case WSAEWOULDBLOCK: return SE_EWOULDBLOCK;
	case WSAEINPROGRESS: return SE_EINPROGRESS;
	case WSAEALREADY: return SE_EALREADY;
	case WSAENOTSOCK: return SE_ENOTSOCK;
	case WSAEDESTADDRREQ: return SE_EDESTADDRREQ;
	case WSAEMSGSIZE: return SE_EMSGSIZE;
	case WSAEPROTOTYPE: return SE_EPROTOTYPE;
	case WSAENOPROTOOPT: return SE_ENOPROTOOPT;
	case WSAEPROTONOSUPPORT: return SE_EPROTONOSUPPORT;
	case WSAESOCKTNOSUPPORT: return SE_ESOCKTNOSUPPORT;
	case WSAEOPNOTSUPP: return SE_EOPNOTSUPP;
	case WSAEPFNOSUPPORT: return SE_EPFNOSUPPORT;
	case WSAEAFNOSUPPORT: return SE_EAFNOSUPPORT;
	case WSAEADDRINUSE: return SE_EADDRINUSE;
	case WSAEADDRNOTAVAIL: return SE_EADDRNOTAVAIL;
	case WSAENETDOWN: return SE_ENETDOWN;
	case WSAENETUNREACH: return SE_ENETUNREACH;
	case WSAENETRESET: return SE_ENETRESET;
	case WSAECONNABORTED: return SE_ECONNABORTED;
	case WSAECONNRESET: return SE_ECONNRESET;
	case WSAENOBUFS: return SE_ENOBUFS;
	case WSAEISCONN: return SE_EISCONN;
	case WSAENOTCONN: return SE_ENOTCONN;
	case WSAESHUTDOWN: return SE_ESHUTDOWN;
	case WSAETOOMANYREFS: return SE_ETOOMANYREFS;
	case WSAETIMEDOUT: return SE_ETIMEDOUT;
	case WSAECONNREFUSED: return SE_ECONNREFUSED;
	case WSAELOOP: return SE_ELOOP;
	case WSAENAMETOOLONG: return SE_ENAMETOOLONG;
	case WSAEHOSTDOWN: return SE_EHOSTDOWN;
	case WSAEHOSTUNREACH: return SE_EHOSTUNREACH;
	case WSAENOTEMPTY: return SE_ENOTEMPTY;
	case WSAEPROCLIM: return SE_EPROCLIM;
	case WSAEUSERS: return SE_EUSERS;
	case WSAEDQUOT: return SE_EDQUOT;
	case WSAESTALE: return SE_ESTALE;
	case WSAEREMOTE: return SE_EREMOTE;
	case WSAEDISCON: return SE_EDISCON;
	case WSASYSNOTREADY: return SE_SYSNOTREADY;
	case WSAVERNOTSUPPORTED: return SE_VERNOTSUPPORTED;
	case WSANOTINITIALISED: return SE_NOTINITIALISED;
	case WSAHOST_NOT_FOUND: return SE_HOST_NOT_FOUND;
	case WSATRY_AGAIN: return SE_TRY_AGAIN;
	case WSANO_RECOVERY: return SE_NO_RECOVERY;
	case WSANO_DATA: return SE_NO_DATA;
		// case : return SE_UDP_ERR_PORT_UNREACH; //@TODO Find it's replacement
	}

	UE_LOGF(LogSockets, Warning, "Unhandled socket error! Error Code: %d", Code);
	check(0);

	return SE_NO_ERROR;
}


bool FSocketSubsystemWindows::HasNetworkDevice()
{
	return true;
}


const TCHAR* FSocketSubsystemWindows::GetSocketAPIName() const
{
	return TEXT("WinSock");
}

FSocketBSD* FSocketSubsystemWindows::InternalBSDSocketFactory(SOCKET Socket, ESocketType SocketType, const FString& SocketDescription, const FName& SocketProtocol)
{
	// return a new socket object
	return new FSocketWindows(Socket, SocketType, SocketDescription, SocketProtocol, this);
}