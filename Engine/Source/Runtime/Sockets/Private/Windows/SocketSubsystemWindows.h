// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SocketSubsystem.h"
#include "BSDSockets/SocketSubsystemBSD.h"
#include "SocketsWindows.h"
#include "SocketSubsystemPackage.h"

/**
 * Windows specific socket subsystem implementation.
 */
class FSocketSubsystemWindows
	: public FSocketSubsystemBSD
{
public:

	/** Default constructor. */
	FSocketSubsystemWindows() :
		bTriedToInit(false)
	{ }

	/** Virtual destructor. */
	virtual ~FSocketSubsystemWindows() = default;

public:

	// FSocketSubsystemBSD overrides

	virtual class FSocket* CreateSocket(const FName& SocketType, const FString& SocketDescription, const FName& ProtocolType) override;
	virtual bool HasNetworkDevice() override;
	virtual ESocketErrors GetLastErrorCode() override;
	virtual bool GetLocalAdapterAddresses( TArray<TSharedPtr<FInternetAddr> >& OutAdresses ) override;
	virtual const TCHAR* GetSocketAPIName() const override;
	virtual bool Init( FString& Error ) override;
	virtual void Shutdown() override;
	virtual ESocketErrors TranslateErrorCode( int32 Code ) override;

PACKAGE_SCOPE:

	/** 
	 * Singleton interface for this subsystem.
	 *
	 * @return the only instance of this subsystem.
	 */
	static FSocketSubsystemWindows* Create();

	/** Performs Windows specific socket clean up. */
	static void Destroy();

protected:

	virtual FSocketBSD* InternalBSDSocketFactory(SOCKET Socket, ESocketType SocketType, const FString& SocketDescription, const FName& SocketProtocol) override;

	/** Holds a flag indicating whether Init() has been called before or not. */
	bool bTriedToInit;

	/** Holds the single instantiation of this subsystem. */
	static FSocketSubsystemWindows* SocketSingleton;
};
