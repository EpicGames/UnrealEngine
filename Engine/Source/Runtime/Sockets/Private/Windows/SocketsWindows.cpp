// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocketsWindows.h"

#if PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS && (PLATFORM_HAS_BSD_SOCKETS || PLATFORM_HAS_BSD_IPV6_SOCKETS)

#include "BSDSockets/IPAddressBSD.h"

/* FSocketBSD overrides
 *****************************************************************************/

bool FSocketWindows::Shutdown(ESocketShutdownMode Mode)
{
	int InternalMode = 0;

	switch (Mode)
	{
		case ESocketShutdownMode::Read:
			InternalMode = SD_RECEIVE;
			break;
		case ESocketShutdownMode::Write:
			InternalMode = SD_SEND;
			break;
		case ESocketShutdownMode::ReadWrite:
			InternalMode = SD_BOTH;
			break;
		default:
			return false;
	}

	return shutdown(Socket, InternalMode) == 0;
}

bool FSocketWindows::RecvFromWithPktInfo(uint8* Data, int32 BufferSize, int32& BytesRead, FInternetAddr& Source, FInternetAddr& Destination, ESocketReceiveFlags::Type Flags)
{
	if (!WSARecvMsg)
	{
		return false;
	}

#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	if (GetProtocol() == FNetworkProtocolTypes::IPv6)
	{
		// Invalid combination
		const FInternetAddrBSD& BSDAddr = static_cast<const FInternetAddrBSD&>(Destination);
		if (BSDAddr.GetProtocolType() != FNetworkProtocolTypes::IPv6)
		{
			return false;
		}
	}
#endif

	bool bSuccess = false;
	const bool bStreamSocket = (SocketType == SOCKTYPE_Streaming);
	const int TranslatedFlags = TranslateFlags(Flags);
	FInternetAddrBSD& BSDAddr = static_cast<FInternetAddrBSD&>(Source);
	SOCKLEN Size = sizeof(sockaddr_storage);
	sockaddr* Addr = (sockaddr*)BSDAddr.GetRawAddr();

	constexpr int32 ControlMsgSize = 1024;

	char ControlMsg[ControlMsgSize];

	WSABUF WSABuf;
	WSABuf.buf = (char*)Data;
	WSABuf.len = BufferSize;

	WSAMSG Msg = {};
	Msg.name = Addr;
	Msg.namelen = Size;
	Msg.lpBuffers = &WSABuf;
	Msg.dwBufferCount = 1;
	Msg.Control.buf = ControlMsg;
	Msg.Control.len = ControlMsgSize;

	DWORD BytesRecvd = 0;

	int Result = WSARecvMsg(Socket, &Msg, &BytesRecvd, nullptr, nullptr);
	if (Result == 0)
	{
		BytesRead = BytesRecvd;
		// iterate each message searching for IP_PKTINFO
		for (cmsghdr* CMsg = CMSG_FIRSTHDR(&Msg); CMsg != nullptr; CMsg = CMSG_NXTHDR(&Msg, CMsg))
		{
			if (CMsg->cmsg_type != IP_PKTINFO)
			{
				continue;
			}
#if PLATFORM_HAS_BSD_IPV6_SOCKETS
			if (GetProtocol() == FNetworkProtocolTypes::IPv6)
			{
				in6_pktinfo* PktInfo = (in6_pktinfo*)WSA_CMSG_DATA(CMsg);
				uint8* DestinationAddr = (uint8*)&PktInfo->ipi6_addr;
				constexpr int32 DestinationAddrSize = 16;
				TArray<uint8> DestinationAddrRaw;
				DestinationAddrRaw.AddUninitialized(DestinationAddrSize);
				for (int32 ByteIndex = 0; ByteIndex < DestinationAddrSize; ++ByteIndex)
				{
					DestinationAddrRaw[ByteIndex] = DestinationAddr[ByteIndex];
				}
				Destination.SetRawIp(DestinationAddrRaw);
			}
			else
			{
#endif
				in_pktinfo* PktInfo = (in_pktinfo*)WSA_CMSG_DATA(CMsg);
				uint8* DestinationAddr = (uint8*)&PktInfo->ipi_addr;

				TArray<uint8> DestinationAddrRaw = { DestinationAddr[0], DestinationAddr[1], DestinationAddr[2], DestinationAddr[3] };
				Destination.SetRawIp(DestinationAddrRaw);
#if PLATFORM_HAS_BSD_IPV6_SOCKETS
			}
#endif
			break;
		}
		// For Streaming sockets, 0 indicates a graceful failure
		bSuccess = !bStreamSocket || (BytesRead > 0);
	}
	else
	{
		// For Streaming sockets, don't treat SE_EWOULDBLOCK as an error as we will potentially retry later
		bSuccess = bStreamSocket && (SocketSubsystem->TranslateErrorCode(Result) == SE_EWOULDBLOCK);
		BytesRead = 0;
	}

	if (bSuccess)
	{
		LastActivityTime = FPlatformTime::Seconds();
	}

	return bSuccess;
}

bool FSocketWindows::SetIpPktInfo(bool bEnable)
{
	if (FSocketBSD::SetIpPktInfo(bEnable))
	{
		if (!WSARecvMsg)
		{
			GUID WSARecvMsg_GUID = WSAID_WSARECVMSG;
			DWORD NumberOfBytes;
			WSAIoctl(Socket, SIO_GET_EXTENSION_FUNCTION_POINTER,
				&WSARecvMsg_GUID, sizeof WSARecvMsg_GUID,
				&WSARecvMsg, sizeof WSARecvMsg,
				&NumberOfBytes, NULL, NULL);
		}

		return WSARecvMsg != nullptr;
	}
	
	return false;
}

#endif	//PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS && (PLATFORM_HAS_BSD_SOCKETS || PLATFORM_HAS_BSD_IPV6_SOCKETS)
