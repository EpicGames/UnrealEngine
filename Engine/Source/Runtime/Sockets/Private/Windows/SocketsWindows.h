// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/PlatformTime.h"


#if PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS && (PLATFORM_HAS_BSD_SOCKETS || PLATFORM_HAS_BSD_IPV6_SOCKETS)

#include "BSDSockets/SocketsBSD.h"
#include "Mswsock.h"

/**
 * Implements a Windows/BSD network socket.
 */
class FSocketWindows
	: public FSocketBSD
{
public:
	FSocketWindows(SOCKET InSocket, ESocketType InSocketType, const FString& InSocketDescription, const FName& InSocketProtocol, ISocketSubsystem* InSubsystem)
		: FSocketBSD(InSocket, InSocketType, InSocketDescription, InSocketProtocol, InSubsystem)
	{ }

	// FSocketBSD overrides

	virtual bool Shutdown(ESocketShutdownMode Mode) override;
	virtual bool SetIpPktInfo(bool bEnable) override;
	virtual bool RecvFromWithPktInfo(uint8* Data, int32 BufferSize, int32& BytesRead, FInternetAddr& Source, FInternetAddr& Destination, ESocketReceiveFlags::Type Flags = ESocketReceiveFlags::None) override;

protected:
	LPFN_WSARECVMSG WSARecvMsg = nullptr;
};


#endif	//PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS && (PLATFORM_HAS_BSD_SOCKETS || PLATFORM_HAS_BSD_IPV6_SOCKETS
