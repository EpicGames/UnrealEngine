// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPAddress.h"
#include "SocketTypes.h"

class FInternetAddr;

enum class EAddressInfoFlags : uint16
{
	/* Return all addresses regardless of spec. */
	AllResults = 1 << 0,
	/* Don't use DNS resolution of the string (AI_NUMERICHOST),
	 * this makes the address resolution non-blocking, requiring
	 * the hostname to be in the form of an ip address already
	 */
	NoResolveHost = 1 << 1,
	/* Don't resolve the service name (requires the service argument to be nullptr
	 * or a string based representation of a number). This is also-nonblocking.
	 */
	NoResolveService = 1 << 2,
	/* Only return addresses that adapters on this machine can use (AI_ADDRCONFIG) */
	OnlyUsableAddresses = 1 << 3,
	/* Return bindable addresses (AI_PASSIVE). Only works if the hostname argument is null. */
	BindableAddress = 1 << 4,
	/* Include the canonical name of the host with the results list */
	CanonicalName = 1 << 5,
	/* Include the fully qualified domain name of the host with the results list */
	FQDomainName = 1 << 6,
	/* Allow for IPv4 mapped IPv6 addresses */
	AllowV4MappedAddresses = 1 << 7,
	/* Get all addresses, but return V4 mapped IPv6 addresses */
	AllResultsWithMapping = AllowV4MappedAddresses | AllResults,
	/* The default value of a hints flag for the platform (typically just 0) */
	Default = 0,
};

ENUM_CLASS_FLAGS(EAddressInfoFlags);

struct FAddressInfoResultData
{
	UE_DEPRECATED(4.23, "Use constructor that supports an FName as the AddressProtocol property is now deprecated. AddressProtocolName supports multiple protocol types as should be used instead.")
	FAddressInfoResultData(TSharedRef<FInternetAddr> InAddr, SIZE_T InAddrLen, ESocketProtocolFamily InProtocol, ESocketType InSocketConfiguration) :
		AddressProtocol(InProtocol),
		AddressProtocolName(NAME_None),
		SocketConfiguration(InSocketConfiguration),
		AddressLen(InAddrLen),
		Address(InAddr)
	{
	}

	FAddressInfoResultData(TSharedRef<FInternetAddr> InAddr, SIZE_T InAddrLen, const FName InProtocol, ESocketType InSocketConfiguration) :
		AddressProtocol(ESocketProtocolFamily::None),
		AddressProtocolName(InProtocol),
		SocketConfiguration(InSocketConfiguration),
		AddressLen(InAddrLen),
		Address(InAddr)
	{
	}

	FName GetSocketTypeName() const
	{
		switch (SocketConfiguration)
		{
		case SOCKTYPE_Datagram:
			return NAME_DGram;
			break;
		case SOCKTYPE_Streaming:
			return NAME_Stream;
			break;
		default:
		case SOCKTYPE_Unknown:
			return NAME_None;
			break;
		}
	}

	bool operator==(const FAddressInfoResultData& rhs) const
	{
		return SocketConfiguration == rhs.SocketConfiguration &&
			AddressProtocolName == rhs.AddressProtocolName &&
			*Address == *(rhs.Address);
	}

	/* The protocol of the address stored */
	ESocketProtocolFamily AddressProtocol;
	/* The protocol name of the address stored */
	FName AddressProtocolName;
	/* Streaming or datagram */
	ESocketType SocketConfiguration;
	/* Length of the returned address data */
	SIZE_T AddressLen;
	/* The address associated with this result */
	TSharedRef<FInternetAddr> Address;
};

struct FAddressInfoResult
{
	FAddressInfoResult(const TCHAR* InHostName, const TCHAR* InServiceName) :
		QueryHostName(InHostName),
		QueryServiceName(InServiceName),
		ReturnCode(SE_NO_DATA)
	{
	}

	/* The hostname/ip that generated these results */
	FString QueryHostName;
	/* The service name that was used in the query */
	FString QueryServiceName;
	/* The Canonical Name of the query (empty unless FQDomainName or CanonicalName are specified) */
	FString CanonicalNameResult;
	/* The return code of the query */
	ESocketErrors ReturnCode;
	/* The list of results */
	TArray<FAddressInfoResultData> Results;
};

typedef TFunction<void(FAddressInfoResult)> FAsyncGetAddressInfoCallback;
