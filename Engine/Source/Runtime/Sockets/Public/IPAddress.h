// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HAL/PlatformAtomics.h"
#include "Misc/AssertionMacros.h"
#include "SocketTypes.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

/**
 * Represents an internet address. All data is in network byte order
 */
class FInternetAddr
{
protected:
	/** Hidden on purpose */
	FInternetAddr()
	{
	}

public:

	virtual ~FInternetAddr() = default; 

	/**
	 * Compares FInternetAddrs together, comparing the logical net addresses (endpoints)
	 * of the data stored, rather than doing a memory comparison like the equality operator does.
	 * If it is not explicitly implemented, this falls back to just doing the same behavior of the
	 * comparison operator.
	 *
	 * @Param InAddr The address to compare with.
	 *
	 * @return true if the endpoint stored in this FInternetAddr is the same as the input.
	 */
	virtual bool CompareEndpoints(const FInternetAddr& InAddr) const
	{
		return *this == InAddr;
	}

	/**
	 * Sets the ip address from a host byte order uint32
	 *
	 * @param InAddr the new address to use (must convert to network byte order)
	 */
	virtual void SetIp(uint32 InAddr) = 0;
	
	/**
	 * Sets the ip address from a string ("A.B.C.D")
	 *
	 * @param InAddr the string containing the new ip address to use
	 */
	virtual void SetIp(const TCHAR* InAddr, bool& bIsValid) = 0;

	/**
	 * Copies the network byte order ip address to a host byte order dword
	 *
	 * @param OutAddr the out param receiving the ip address
	 */
	virtual void GetIp(uint32& OutAddr) const = 0;

	/**
	 * Sets the port number from a host byte order int
	 *
	 * @param InPort the new port to use (must convert to network byte order)
	 */
	virtual void SetPort(int32 InPort) = 0;

	/** Report whether the port is in a valid range for SetPort. */
	virtual bool IsPortValid(int32 InPort) const { return true; }

	/**
	 * Copies the port number from this address and places it into a host byte order int
	 *
	 * @param OutPort the host byte order int that receives the port
	 */
	virtual void GetPort(int32& OutPort) const
	{
		OutPort = GetPort();
	}

	/**
	 * Returns the port number from this address in host byte order
	 */
	virtual int32 GetPort() const = 0;

	/** 
	 * Set Platform specific port data
	 */
	virtual void SetPlatformPort(int32 InPort)
	{
		SetPort(InPort);
	}

	/** 
	 * Get platform specific port data.
	 */
	virtual int32 GetPlatformPort() const
	{
		return GetPort();
	}

	/**
	 * Sets the ip address from a raw network byte order array.
	 *
	 * @param RawAddr the new address to use (must be converted to network byte order)
	 */
	virtual void SetRawIp(const TArray<uint8>& RawAddr) = 0;

	/**
	 * Gets the ip address in a raw array stored in network byte order.
	 *
	 * @return The raw address stored in an array
	 */
	virtual TArray<uint8> GetRawIp() const = 0;

	/** Sets the address to be any address */
	virtual void SetAnyAddress() = 0;

	/** Sets the address to broadcast */
	virtual void SetBroadcastAddress() = 0;

	/** Sets the address to loopback */
	virtual void SetLoopbackAddress() = 0;

	/**
	 * Converts this internet ip address to string form
	 *
	 * @param bAppendPort whether to append the port information or not
	 */
	virtual FString ToString(bool bAppendPort) const = 0;

	/**
	 * Compares two internet ip addresses for equality
	 *
	 * @param Other the address to compare against
	 */
	virtual bool operator==(const FInternetAddr& Other) const
	{
		TArray<uint8> ThisIP = GetRawIp();
		TArray<uint8> OtherIP = Other.GetRawIp();
		return ThisIP == OtherIP && GetPort() == Other.GetPort();
	}

	/**
	 * Hash function for use with TMap's - exposed through FInternetAddrMapRef
	 *
	 * @return	The value to use for the hash
	 */
	virtual uint32 GetTypeHash() const = 0;

	/**
	 * Is this a well formed internet address
	 *
	 * @return true if a valid IP, false otherwise
	 */
	virtual bool IsValid() const = 0;
	
	/**
	 * Creates a new structure with the same data as this structure
	 *
	 * @return The new structure
	 */
	virtual TSharedRef<FInternetAddr> Clone() const = 0;

	/**
	 * Returns the protocol type name of the address data currently stored in this struct
	 *
	 * @return The type of the address. 
	 *         If it's not known or overridden in a derived class, the return is None.
	 */
	virtual FName GetProtocolType() const
	{
		return NAME_None;
	}

	SOCKETS_API virtual void DumpAddrData() const;

	friend inline uint32 GetTypeHash(const FInternetAddr& InAddr)
	{
		return InAddr.GetTypeHash();
	}
};


/**
 * KeyFuncs for mapping/hashing FInternetAddr shared references
 *
 * NOTE: Implements TSharedRef over TSharedPtr, as this is required for performance critical code
 */
template<typename ValueType>
struct FInternetAddrKeyMapFuncs : public BaseKeyFuncs<ValueType, TSharedRef<FInternetAddr>, false>
{
	static inline const TSharedRef<FInternetAddr>& GetSetKey(const TPair<TSharedRef<FInternetAddr>, ValueType>& Element)
	{
		return Element.Key;
	}

	static inline bool Matches(const TSharedRef<FInternetAddr>& A, const TSharedRef<FInternetAddr>& B)
	{
		return *A == *B;
	}

	static inline uint32 GetKeyHash(const TSharedRef<FInternetAddr>& Key)
	{
		return Key->GetTypeHash();
	}
};

template<typename ValueType>
struct FInternetAddrConstKeyMapFuncs : public BaseKeyFuncs<ValueType, TSharedRef<const FInternetAddr>, false>
{
	static inline const TSharedRef<const FInternetAddr>& GetSetKey(const TPair<TSharedRef<const FInternetAddr>, ValueType>& Element)
	{
		return Element.Key;
	}

	static inline bool Matches(const TSharedRef<const FInternetAddr>& A, const TSharedRef<const FInternetAddr>& B)
	{
		return *A == *B;
	}

	static inline uint32 GetKeyHash(const TSharedRef<const FInternetAddr>& Key)
	{
		return Key->GetTypeHash();
	}
};
