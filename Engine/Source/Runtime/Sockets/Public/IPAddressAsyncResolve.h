// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/AsyncWork.h"
#include "IPAddress.h"
#include "Stats/Stats.h"

/**
 * Abstract interface used by clients to get async host name resolution to work in a
 * cross-platform way
 */
class FResolveInfo
{
protected:
	/** Hidden on purpose */
	FResolveInfo()
	{
	}

public:
	/** Virtual destructor for child classes to overload */
	virtual ~FResolveInfo() = default;

	/**
	 * Whether the async process has completed or not
	 *
	 * @return true if it completed successfully, false otherwise
	 */
	virtual bool IsComplete() const = 0;

	/**
	 * The error that occurred when trying to resolve
	 *
	 * @return error code from the operation
	 */
	virtual int32 GetErrorCode() const = 0;

	/**
	 * Returns a copy of the resolved address
	 *
	 * @return the resolved IP address
	 */
	virtual const FInternetAddr& GetResolvedAddress() const = 0;
};

/** A non-async resolve info for returning cached results */
class FResolveInfoCached : public FResolveInfo
{
protected:
	/** The address that was resolved */
	TSharedPtr<FInternetAddr> Addr;

	/** Hidden on purpose */
	FResolveInfoCached() {}

public:
	/**
	 * Sets the address to return to the caller
	 *
	 * @param InAddr the address that is being cached
	 */
	FResolveInfoCached(const FInternetAddr& InAddr);

	// FResolveInfo interface

		/**
		 * Whether the async process has completed or not
		 *
		 * @return true if it completed successfully, false otherwise
		 */
	virtual bool IsComplete() const
	{
		return true;
	}

	/**
	 * The error that occurred when trying to resolve
	 *
	 * @return error code from the operation
	 */
	virtual int32 GetErrorCode() const
	{
		return 0;
	}

	/**
	 * Returns a copy of the resolved address
	 *
	 * @return the resolved IP address
	 */
	virtual const FInternetAddr& GetResolvedAddress() const
	{
		return *Addr;
	}
};

//
// Class for creating a background thread to resolve a host.
//
class FResolveInfoAsync :
	public FResolveInfo
{
	//
	// A simple wrapper task that calls back to FResolveInfoAsync to do the actual work
	//
	class FResolveInfoAsyncWorker
	{
	public:
		/** Pointer to FResolveInfoAsync to call for async work*/
		FResolveInfoAsync* Parent;

		/** Constructor
		* @param InParent the FResolveInfoAsync to route the async call to
		*/
		FResolveInfoAsyncWorker(FResolveInfoAsync* InParent)
			: Parent(InParent)
		{
		}

		/** Call DoWork on the parent */
		void DoWork()
		{
			Parent->DoWork();
		}

		inline TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FResolveInfoAsyncWorker, STATGROUP_ThreadPoolAsyncTasks);
		}

		/** Indicates to the thread pool that this task is abandonable */
		bool CanAbandon()
		{
			return true;
		}

		/** Effects the ending of the async resolve */
		void Abandon()
		{
			FPlatformAtomics::InterlockedExchange(&Parent->bShouldAbandon, true);
		}
	};
	// Variables.
	TSharedPtr<FInternetAddr>	Addr;
	ANSICHAR	HostName[256];
	/** Error code returned by GetHostByName. */
	int32			ErrorCode;
	/** Tells the worker thread whether it should abandon it's work or not */
	volatile int32 bShouldAbandon;
	/** Async task for async resolve */
	FAsyncTask<FResolveInfoAsyncWorker> AsyncTask;

public:
	/**
	 * Copies the host name for async resolution
	 *
	 * @param InHostName the host name to resolve
	 */
	FResolveInfoAsync(const ANSICHAR* InHostName);

	/**
	 * Start the async work and perform it synchronously if no thread pool is available
	 */
	void StartAsyncTask()
	{
		check(AsyncTask.GetTask().Parent == this); // need to make sure these aren't memcpy'd around after construction
		AsyncTask.StartBackgroundTask();
	}

	/**
	 * Resolves the specified host name
	 */
	void DoWork();

	// FResolveInfo interface

		/**
		 * Whether the async process has completed or not
		 *
		 * @return true if it completed successfully, false otherwise
		 */
	virtual bool IsComplete() const
	{
		// this semantically const, but IsDone syncs the async task, and that causes writes
		return const_cast<FAsyncTask<FResolveInfoAsyncWorker> &>(AsyncTask).IsDone();
	}

	/**
	 * The error that occurred when trying to resolve
	 *
	 * @return error code from the operation
	 */
	virtual int32 GetErrorCode() const
	{
		return ErrorCode;
	}

	/**
	 * Returns a copy of the resolved address
	 *
	 * @return the resolved IP address
	 */
	virtual const FInternetAddr& GetResolvedAddress() const
	{
		return *Addr;
	}
};
