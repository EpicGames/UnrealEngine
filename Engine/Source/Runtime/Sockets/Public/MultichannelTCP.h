// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()

#include "SocketTypes.h"
#include "SocketSubsystem.h"
#include "IPAddress.h"
#include "Sockets.h"

#include "MultichannelTcpGlobals.h"
#include "NetworkMessage.h"
#include "MultichannelTcpReceiver.h"
#include "MultichannelTcpSender.h"
#include "MultichannelTcpSocket.h"
