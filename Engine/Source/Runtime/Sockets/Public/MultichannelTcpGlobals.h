// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"

SOCKETS_API DECLARE_LOG_CATEGORY_EXTERN(LogMultichannelTCP, Log, All);


/** Magic number used to verify packet header **/
enum
{
	MultichannelMagic = 0xa692339f
};
