// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NetworkMessage.h"
#include "HAL/RunnableThread.h"
#include "HAL/Runnable.h"
#include "MultichannelTcpGlobals.h"

class Error;
class FSocket;

/**
 * Declares a delegate to be invoked when data has been received.
 *
 * The first parameter will hold the received data.
 * The second parameter is the number of bytes received.
 * The third parameter indicates whether byte swapping is needed on the received data.
 */
DECLARE_DELEGATE_ThreeParams(FOnMultichannelTcpReceive, const TArray<uint8>&, uint32, bool);


/**
 * Implements a receiver for multichannel TCP sockets.
 */
class FMultichannelTcpReceiver
	: public FRunnable
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InSocket The socket to receive from.
	 * @param InReceiveDelegate Delete to handle data as it becomes available, called from the receive thread.
	 */
	FMultichannelTcpReceiver( FSocket* InSocket, const FOnMultichannelTcpReceive& InReceiveDelegate )
		: Socket(InSocket)
		, ReceiveDelegate(InReceiveDelegate)
	{
		Thread = FRunnableThread::Create(this, TEXT("FMultichannelTCPReceiver"), 8 * 1024, TPri_AboveNormal);
	}

	/** Destructor. */
	~FMultichannelTcpReceiver()
	{
		if (Thread)
		{
			Thread->Kill(true);
			delete Thread;
		}
	}

public:

	/**
	 * Gets the number of payload bytes actually sent to the socket.
	 */
	int32 GetBytesReceived()
	{
		return BytesReceived;
	}

public:

	// FRunnable interface

	virtual bool Init( )
	{
		return true;
	}

	virtual uint32 Run( )
	{
		while (1)
		{
			// read a header and payload pair
			FArrayReader Payload; 

			if (FNFSMessageHeader::ReceivePayload(Payload, FSimpleAbstractSocket_FSocket(Socket)) == false)
			{
				// if we failed to receive the payload, then the client is most likely dead, so, we can kill this connection
				// @todo: Add more error codes, maybe some errors shouldn't kill the connection
				break;
			}

			uint32 Magic;

			Payload << Magic;

			if (Magic != MultichannelMagic)
			{
				UE_LOGF(LogMultichannelTCP, Error, "Wrong magic.");

				break;
			}

			uint32 Channel;
			Payload << Channel;
			
			TArray<uint8> InnerPayload;
			Payload << InnerPayload;

			ReceiveDelegate.Execute(InnerPayload, Channel, Payload.ForceByteSwapping());

			FPlatformAtomics::InterlockedAdd(&BytesReceived, InnerPayload.Num());
		}

		return 0;
	}

	virtual void Stop( ) { }
	virtual void Exit( ) { }

private:

	/** Holds the number of bytes received so far. */
	int32 BytesReceived;

	/** Holds the socket to use for communication. */
	FSocket* Socket;

	/** Holds the thread we are running on. */
	FRunnableThread* Thread;

private:

	/** Holds a delegate to be invoked when data has been received. */
	FOnMultichannelTcpReceive ReceiveDelegate;
};
