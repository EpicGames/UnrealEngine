// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/BufferArchive.h"
#include "NetworkMessage.h"
#include "Misc/ScopedEvent.h"
#include "Misc/ScopeLock.h"
#include "HAL/RunnableThread.h"
#include "HAL/Runnable.h"
#include "MultichannelTcpGlobals.h"

class Error;
class FSocket;

/**
 * Declares a delegate to be invoked when checking if bandwidth permits sending a packet
 *
 * The first parameter is the size of the packet to send.
 * The second parameter is the channel that the packet would be sent on.
 */
DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnMultichannelTcpOkToSend, int32, uint32);


/**
 * Implements a sender for multichannel TCP sockets.
 */
class FMultichannelTcpSender
	: public FRunnable
{
	enum
	{
		/** Defines the maximum payload size per packet (in bytes). */
		MaxPacket = 128 * 1024 - 8
	};

public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InSocket The socket to send to.
	 * @param InOkToSendDelegate Delegate to ask if it is ok to send a packet.
	 */
	FMultichannelTcpSender( FSocket* InSocket, const FOnMultichannelTcpOkToSend& InOkToSendDelegate )
		: BytesSent(0)
		, Socket(InSocket)
		, OkToSendDelegate(InOkToSendDelegate)
	{
		Thread = FRunnableThread::Create(this, TEXT("FMultichannelTCPSender"), 8 * 1024, TPri_AboveNormal);
	}

	/** Destructor. */
	~FMultichannelTcpSender(  )
	{
		if (Thread)
		{
			Thread->Kill(true);
			delete Thread;
		}
	}

public:

	/** Call when bandwidth tests should be retried, possibly sending data if there is available bandwidth. */
	void AttemptResumeSending( )
	{
		FScopeLock ScopeLock(&SendBuffersCriticalSection);
		AttemptResumeSendingInternal();
	}

	/**
	 * Gets the number of payload bytes actually sent to the socket.
	 */
	int32 GetBytesSent()
	{
		return BytesSent;
	}

	/**
	 * Sends data through the given channel.
	 *
	 * This method does not block on bandwidth and never fails.
	 *
	 * @param Data The buffer containing the data to send.
	 * @param Count The number of bytes to send from the buffer.
	 */
	void Send( const uint8* Data, int32 Count, uint32 Channel )
	{
		FScopeLock ScopeLock(&SendBuffersCriticalSection);
		check(Count);

		TArray<uint8>* SendBuffer = SendBuffers.FindRef(Channel);

		if (!SendBuffer)
		{
			SendBuffer = new TArray<uint8>();
			SendBuffers.Add(Channel, SendBuffer);
		}

		int32 OldCount = SendBuffer->Num();

		SendBuffer->AddUninitialized(Count);

		FMemory::Memcpy(SendBuffer->GetData() + OldCount, Data, Count);

		AttemptResumeSendingInternal();
	}

public:

	// FRunnable interface

	virtual bool Init( )
	{
		return true;
	}

	virtual uint32 Run( )
	{
		while (1)
		{
			TArray<uint8> Data;
			uint32 Channel = 0;

			FScopedEvent* LocalEventToRestart = NULL;
			{
				FScopeLock ScopeLock(&SendBuffersCriticalSection);

				TArray<uint8>* SendBuffer = NULL;

				for (TMap<uint32, TArray<uint8>*>::TIterator It(SendBuffers); It; ++It)
				{
					check(It.Value()->Num());

					if (!SendBuffer || It.Key() < Channel)
					{
						Channel = It.Key();
						SendBuffer = It.Value();
					}
				}

				if (SendBuffer)
				{
					int32 Num = SendBuffer->Num();
					check(Num > 0);

					int32 Size = FMath::Min<int32>(Num, MaxPacket);

					if (OkToSendDelegate.Execute(Size, Channel))
					{
						Data.AddUninitialized(Size);
						FMemory::Memcpy(Data.GetData(), SendBuffer->GetData(), Size);

						if (Size < Num)
						{
							SendBuffer->RemoveAt(0, Size);
						}
						else
						{
							delete SendBuffer;

							SendBuffers.Remove(Channel);
						}

						check(Data.Num());
					}
				}
				else
				{
					LocalEventToRestart = new FScopedEvent();
					EventToRestart = LocalEventToRestart;
				}
			}

			if (Data.Num())
			{
				FBufferArchive Ar;

				uint32 Magic = MultichannelMagic;

				Ar << Magic;
				Ar << Channel;
				Ar << Data;

				if (!FNFSMessageHeader::WrapAndSendPayload(Ar, FSimpleAbstractSocket_FSocket(Socket)))
				{
					UE_LOGF(LogMultichannelTCP, Error, "Failed to send payload.");

					break;
				}

				FPlatformAtomics::InterlockedAdd(&BytesSent, Data.Num());
			}

			delete LocalEventToRestart; // block here if we don't have any data
		}

		return 0;
	}

	virtual void Stop( ) { }

	virtual void Exit( )
	{
		FScopeLock ScopeLock(&SendBuffersCriticalSection);

		for (TMap<uint32, TArray<uint8>*>::TIterator It(SendBuffers); It; ++It)
		{
			delete It.Value();
		}
	}

protected:

	/** Internal call similar to AttemptResumeSending, but does not do the requisite lock. */
	void AttemptResumeSendingInternal( )
	{
		if (EventToRestart)
		{
			FScopedEvent* LocalEventToRestart = EventToRestart;
			EventToRestart = NULL;
			LocalEventToRestart->Trigger();
		}
	}

private:

	/** Holds the total number of payload bytes sent to the socket. */
	int32 BytesSent;

	/** Holds an event to trigger when bandwidth has freed up. */
	FScopedEvent* EventToRestart;

	/** Holds the buffers to hold pending per-channel data. */
	TMap<uint32, TArray<uint8>*> SendBuffers;

	/** Holds the critical section to guard the send buffers. */
	FCriticalSection SendBuffersCriticalSection;

	/** Holds the socket to use. */
	FSocket* Socket;

	/** Holds the thread that is running this instance. */
	FRunnableThread* Thread;

private:

	/** Holds a delegate to be invoked when checking if there is sufficient bandwidth to send. */
	FOnMultichannelTcpOkToSend OkToSendDelegate;
};
