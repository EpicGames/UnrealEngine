// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/BufferArchive.h"
#include "Misc/ScopedEvent.h"
#include "Misc/ScopeLock.h"
#include "Serialization/MemoryReader.h"
#include "MultichannelTcpReceiver.h"
#include "MultichannelTcpSender.h"

class FSocket;

/**
 * Class to multiplex several streams on a single TCP socket.
 *
 * The primary feature here is to allow blocking reads to multiple channels simultaneously
 * without interference. Generally one of these is created on both sides of the connection,
 * immediately after the connection is established
 */
class FMultichannelTcpSocket
{
	enum
	{
		/**
		 * Defines the control channel.
		 *
		 * The control channel is for Acks, highest priority and not subject to bandwidth limits.
		 */
		ControlChannel = 0
	};

public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InSocket The underlying socket to use.
	 * @param InBandwidthLatencyProduct The maximum amount of unacknowledged data to send.
	 */
	FMultichannelTcpSocket( FSocket* InSocket, uint64 InBandwidthLatencyProduct )
		: BandwidthLatencyProduct(InBandwidthLatencyProduct)
		, RemoteReceiverBytesReceived(0)
		, Receiver(InSocket, FOnMultichannelTcpReceive::CreateRaw(this, &FMultichannelTcpSocket::HandleReceiverReceive))
		, Sender(InSocket, FOnMultichannelTcpOkToSend::CreateRaw(this, &FMultichannelTcpSocket::HandleSenderOkToSend))
		, Socket(InSocket)
	{ }

public:

	/**
	 * Block until data is available to receive.
	 *
	 * Can be called from any thread, but not multiple threads for one channel at once.
	 *
	 * @param Data The buffer to fill.
	 * @param Count The number of bytes to receive.
	 * @param Channel The channel to receive on.
	 */
	int32 BlockingReceive( uint8* Data, int32 Count, uint32 Channel )
	{
		check((Channel != ControlChannel) && Data); // Channel 0 is reserved, must want data
		
		if (Count > 0)
		{
			for (int32 Attempt = 0; Attempt < 2; Attempt++)
			{
				FScopedEvent* LocalEventToRestart = NULL;
				{
					FScopeLock ScopeLock(&ReceiveBuffersCriticalSection);

					FReceiveBuffer* ChannelBuffer = ReceiveBuffers.FindRef(Channel);

					if (!ChannelBuffer)
					{
						ChannelBuffer = new FReceiveBuffer();

						ReceiveBuffers.Add(Channel, ChannelBuffer);
					}

					// Would be bad to have multiple listeners
					check(!ChannelBuffer->EventToResumeWhenDataIsReady && !ChannelBuffer->BytesRequiredToResume);
				
					if (ChannelBuffer->Buffer.Num() >= Count)
					{
						FMemory::Memcpy(Data, ChannelBuffer->Buffer.GetData(), Count);

						if (ChannelBuffer->Buffer.Num() == Count)
						{
							ReceiveBuffers.Remove(Channel);
						}
						else
						{
							ChannelBuffer->Buffer.RemoveAt(0, Count);
						}

						return Count;
					}

					if (!Attempt) // if someone woke us up with insufficient data, we simply error out
					{
						LocalEventToRestart = new FScopedEvent();

						ChannelBuffer->EventToResumeWhenDataIsReady = LocalEventToRestart;
						ChannelBuffer->BytesRequiredToResume = Count;
					}
				}

				delete LocalEventToRestart; // wait here for sufficient data
			}
		}

		return 0;
	}

	/**
	 * Non-blocking test of available data.
	 *
	 * Can be called from any thread, but realize that if this returns > 0, another thread could steal the data.
	 *
	 * @param Channel The channel to check.
	 * @return The number of bytes in the receive buffer.
	 */
	int32 DataAvailable( uint32 Channel )
	{
		FScopeLock ScopeLock(&ReceiveBuffersCriticalSection);

		FReceiveBuffer* ChannelBuffer = ReceiveBuffers.FindRef(Channel);

		return ChannelBuffer ? ChannelBuffer->Buffer.Num() : 0;
	}

	/**
	 * Non-blocking return of available data.
	 *
	 * Can be called from any thread, but realize that multiple threads hammering
	 * a channel at once is unlikely to give useful results.
	 *
	 * @param Data The buffer to hold the results, if any.
	 * @param MaxCount the number of bytes in the receive buffer.
	 * @param Channel The channel to check.
	 * @return The number of bytes written into Data.
	 */
	int32 PollingReceive( uint8* Data, int32 MaxCount, uint32 Channel )
	{
		check(Channel != ControlChannel && Data && MaxCount > 0); // Channel 0 is reserved, must want data

		FScopeLock ScopeLock(&ReceiveBuffersCriticalSection);

		FReceiveBuffer* ChannelBuffer = ReceiveBuffers.FindRef(Channel);
		
		if (!ChannelBuffer)
		{
			return 0;
		}

		check(!ChannelBuffer->EventToResumeWhenDataIsReady && !ChannelBuffer->BytesRequiredToResume && ChannelBuffer->Buffer.Num()); // would be bad to have multiple listeners
		
		int32 Count = FMath::Min<int32>(ChannelBuffer->Buffer.Num(), MaxCount);

		FMemory::Memcpy(Data, ChannelBuffer->Buffer.GetData(), Count);
		
		if (ChannelBuffer->Buffer.Num() == Count)
		{
			ReceiveBuffers.Remove(Channel);
		}
		else
		{
			ChannelBuffer->Buffer.RemoveAt(0, Count);
		}

		return Count;
	}

	/**
	 * Send data out a given channel, this does not block on bandwidth, and never fails.
	 *
	 * Can be called from any thread, but if you are calling from multiple threads,
	 * make sure you are sending an atomic unit.
	 *
	 * @param Data The buffer containing the data to send.
	 * @param Count The number of bytes to send.
	 * @param Channel The channel to send on.
	 */
	void Send( const uint8* Data, int32 Count, uint32 Channel )
	{
		check((Channel != ControlChannel) && Data && Count); // Channel 0 is reserved, must have data
		Sender.Send(Data, Count, Channel);
	}

private:

	/** Helper struct for a receive buffer per channel **/
	struct FReceiveBuffer
	{
		/** buffer of data not yet accepted by anyone **/
		TArray<uint8> Buffer;

		/** number of bytes a thread is blocked, waiting to read (or zero if nobody is blocked) **/
		int32 BytesRequiredToResume;

		/** Event to release some thread, once we have at least BytesRequiredToResume **/
		FScopedEvent* EventToResumeWhenDataIsReady;

		/** Constructor, sets the "nobody is waiting" state **/
		FReceiveBuffer( )
			: BytesRequiredToResume(0)
			, EventToResumeWhenDataIsReady(NULL)
		{ }
	};

	// Callback for receiving data from the receiver thread.
	void HandleReceiverReceive( const TArray<uint8>& Payload, uint32 Channel, bool bForceByteswapping )
	{
		if (Channel == ControlChannel)
		{
			FMemoryReader Ar(Payload);
			
			Ar.SetByteSwapping(bForceByteswapping);
			
			uint64 RemoteBytesReceived;
			Ar << RemoteBytesReceived;
			
			RemoteReceiverBytesReceived = RemoteBytesReceived;
			
			Sender.AttemptResumeSending();
			
			return;
		}

		// process the actual payload
		{
			FScopeLock ScopeLock(&ReceiveBuffersCriticalSection);
			FReceiveBuffer* ChannelBuffer = ReceiveBuffers.FindRef(Channel);
			
			if (!ChannelBuffer)
			{
				ChannelBuffer = new FReceiveBuffer();
				ReceiveBuffers.Add(Channel, ChannelBuffer);
			}
			
			ChannelBuffer->Buffer += Payload;
			
			if (ChannelBuffer->Buffer.Num() >= ChannelBuffer->BytesRequiredToResume && ChannelBuffer->EventToResumeWhenDataIsReady)
			{
				ChannelBuffer->EventToResumeWhenDataIsReady->Trigger();
				ChannelBuffer->EventToResumeWhenDataIsReady = NULL;
				ChannelBuffer->BytesRequiredToResume = 0;
			}
		}

		// send a control channel message back
		{
			FBufferArchive Ar;
			uint64 BytesReceived = Receiver.GetBytesReceived();
			Ar << BytesReceived;
			Sender.Send(Ar.GetData(), Ar.Num(), ControlChannel);
		}
	}

	/** Callback for checking if the sender thread is permitted to send a packet. */
	bool HandleSenderOkToSend( int32 PayloadSize, uint32 Channel )
	{
		if (Channel == ControlChannel)
		{
			return true; // always ok to send on the control channel
		}

		bool Result = (Sender.GetBytesSent() + PayloadSize) < (RemoteReceiverBytesReceived + BandwidthLatencyProduct);
		
		return Result;
	}

private:

	/** Holds the maximum amount of unacknowledged data to send. */
	uint64 BandwidthLatencyProduct;

	/** Holds the buffers for incoming per-channel data. */
	TMap<uint32, FReceiveBuffer*> ReceiveBuffers;

	/** Holds a critical section to guard the receive buffers. */
	FCriticalSection ReceiveBuffersCriticalSection;

	/** Holds the total number of bytes received by the client (comes from an 'Ack' on the control channel). */
	int64 RemoteReceiverBytesReceived;

	/** Holds the receiver thread. */
	FMultichannelTcpReceiver Receiver;

	/** Holds the sender thread. */
	FMultichannelTcpSender Sender;

	/** Holds the socket used for actual transfers. */
	FSocket* Socket;
};
