// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ByteSwap.h"
#include "Misc/Crc.h"
#include "Misc/DateTime.h"
#include "Misc/EnumClassFlags.h"
#include "Serialization/Archive.h"
#include "Serialization/ArrayReader.h"
#include "Serialization/BufferArchive.h"

class FArrayReader;
class FString;

enum
{ 
	DEFAULT_TCP_FILE_SERVING_PORT=41899, // default port to use when making file server tcp connections (used if no protocol is specified)
	DEFAULT_HTTP_FILE_SERVING_PORT=41898 // port that the network file server uses
};

// transitional define. Set to zero to avoid using multichannel TCP for the file server and client.
#define USE_MCSOCKET_FOR_NFS (0)

// Message commands, these correspond to the operations of the low level file system
namespace NFS_Messages
{
	enum Type
	{
		SyncFile,
		DeleteFile,
		MoveFile,
		SetReadOnly,
		OpenRead,
		OpenWrite,
		OpenAppend,
		CreateDirectory,
		DeleteDirectory,
		IterateDirectory,
		IterateDirectoryRecursively,
		DeleteDirectoryRecursively,		
		CopyFile,
		GetFileInfo,
		Read,
		ReadAt,
		Write,
		Close,
		Seek,
		SetTimeStamp,
		ToAbsolutePathForRead,
		ToAbsolutePathForWrite,
		ReportLocalFiles,
		GetFileList,
		Heartbeat,
		RecompileShaders,
	};
}

// Reserved channels for the network file system over multichannel tcp
namespace NFS_Channels
{
	enum Type
	{
		Main = 100,
		UnsolicitedFiles = 101,
		Heatbeat = 102,
	};
}



enum class EConnectionFlags : uint8
{
	None = 0x00000000,
	Streaming = 0x00000001,
	PreCookedIterative = 0x00000002, //DEPRECATED
};
ENUM_CLASS_FLAGS(EConnectionFlags);



/**
* Simple abstraction for sockets that allows FNFSMessageHeader to use either an ordinary socket or a mutichannel socket
**/
class FSimpleAbstractSocket
{
public:
	/**
	* Block until we receive data from the socket
	* @param Results		Buffer of at least Size size, used to hold the results
	* @param Size			Number of bytes to read
	* @return				true if read is successful
	**/
	virtual bool Receive(uint8 *Results, int32 Size) const = 0;
	/**
	* Send bytes out the socket
	* @param Buffer			Buffer containing bytes to send
	* @param Size			Number of bytes to send
	* @return				true if read is successful
	**/
	virtual bool Send(const uint8 *Buffer, int32 Size) const = 0;
	/** return the magic number for this message, also used for endian correction on the archives **/
	virtual uint32 GetMagic() const = 0;
	/** Destructor */
	virtual ~FSimpleAbstractSocket() = default;
};

/**
* Ordinary socket version of FSimpleAbstractSocket
**/
class FSimpleAbstractSocket_FSocket : public FSimpleAbstractSocket
{
	/** Ordinary socket to forward requests to **/
	class FSocket* Socket;
public:
	/**
	* Constructor
	* @param InSocket		Ordinary socket to forward requests to
	**/
	FSimpleAbstractSocket_FSocket(class FSocket* InSocket)
		: Socket(InSocket)
	{
	}
	SOCKETS_API virtual bool Receive(uint8 *Results, int32 Size) const;
	SOCKETS_API virtual bool Send(const uint8 *Buffer, int32 Size) const;
	virtual uint32 GetMagic() const
	{
		return 0x9E2B83C1;
	}
};

/**
* Multichannel socket version of FSimpleAbstractSocket
**/
class FSimpleAbstractSocket_FMultichannelTCPSocket : public FSimpleAbstractSocket
{
	/** Multichannel socket to forward requests to **/
	class FMultichannelTcpSocket* Socket;
	/** Channel to send to **/
	uint32 SendChannel;
	/** Channel to receive from **/
	uint32 ReceiveChannel;
public:
	/**
	* Constructor
	* @param InSocket			Multichannel socket to forward requests to
	* @param InSendChannel		Channel to send to
	* @param InReceiveChannel	Channel to receive from, defaults to the sending channel
	**/
	FSimpleAbstractSocket_FMultichannelTCPSocket(class FMultichannelTcpSocket* InSocket, uint32 InSendChannel, uint32 InReceiveChannel = 0)
		: Socket(InSocket)
		, SendChannel(InSendChannel)
		, ReceiveChannel(InReceiveChannel ? InReceiveChannel : InSendChannel)
	{
		check(SendChannel);
		check(ReceiveChannel);
	}
	SOCKETS_API virtual bool Receive(uint8 *Results, int32 Size) const;
	SOCKETS_API virtual bool Send(const uint8 *Buffer, int32 Size) const;
	virtual uint32 GetMagic() const
	{
		return 0x9E2B83C2;
	}
};

/**
* Simple wrapper for sending and receiving atomic packets
**/
struct FNFSMessageHeader
{
	/** Magic number, used for error checking and endianess checking **/
	uint32 Magic;
	/** Size of payload **/
	uint32 PayloadSize;
	/** CRC of payload **/
	uint32 PayloadCrc;

	/** Constructor for empty header */
	FNFSMessageHeader(const FSimpleAbstractSocket& InSocket)
		: Magic(InSocket.GetMagic())
		, PayloadSize(0)
		, PayloadCrc(0)
	{
	}

	/** Constructor for a header of a given payload */
	FNFSMessageHeader(const FSimpleAbstractSocket& InSocket, const TArray<uint8>& Payload)
		: Magic(InSocket.GetMagic())
	{
		// make a header for the given payload
		PayloadSize = Payload.Num();
		check(PayloadSize);
		PayloadCrc = FCrc::MemCrc_DEPRECATED(Payload.GetData(), Payload.Num());
	}

	/** Serializer for header **/
	friend FArchive& operator<<(FArchive& Ar, FNFSMessageHeader& Header)
	{
		uint32 DesiredMagic = Header.Magic;
		Ar << Header.Magic;
		if (Ar.IsLoading())
		{
			check(DesiredMagic);
			if (Header.Magic != DesiredMagic)
			{
				uint32 DesiredMagicSwapped = BYTESWAP_ORDER32(DesiredMagic);
				check(DesiredMagic != DesiredMagicSwapped);
				if (Header.Magic == DesiredMagicSwapped)
				{
					Ar.SetByteSwapping(!Ar.ForceByteSwapping());
					Header.Magic = DesiredMagic;
				}
			}
		}
		if (Header.Magic == DesiredMagic)
		{
			Ar << Header.PayloadSize;
			Ar << Header.PayloadCrc;
		}
		return Ar;
	}


	/**
	 * This function will create a header for the payload, then send the header and 
	 * payload over the network
	 *
	 * @param Payload Bytes to send over the network
	 * @param Socket Connection to send the header and payload 
	 *
	 * @return true if successful
	 */
	static SOCKETS_API bool WrapAndSendPayload(const TArray<uint8>& Payload, const FSimpleAbstractSocket& Socket);

	/**
	 * This function will receive a header, and then the payload array from the network
	 * 
	 * @param OutPayload The archive to read into (the response is APPENDed to any data in the archive already, and the archive will be seeked to the start of new data)
	 * @param Socket The socket to read from
	 *
	 * @return true if successful
	 */
	static SOCKETS_API bool ReceivePayload(FArrayReader& OutPayload, const FSimpleAbstractSocket& Socket);

	/**
	 * This function will send a payload data (with header) and wait for a response, serializing
	 * the response to a FBufferArchive
	 *
	 * @param Payload Bytes to send over the network
	 * @param Response The archive to read the response into
	 * @param Socket Connection to send the header and payload 
	 *
	 * @return true if successful
	 */
	static SOCKETS_API bool SendPayloadAndReceiveResponse(const TArray<uint8>& Payload, class FArrayReader& Response, const FSimpleAbstractSocket& Socket);

};


/**
 * A helper class for storing all available file info.
 */
struct FFileInfo
{
	bool FileExists;
	bool ReadOnly;
	FDateTime TimeStamp;
	FDateTime AccessTimeStamp;
	int64 Size;

	FFileInfo()
		: FileExists(false)
		, ReadOnly(false)
		, TimeStamp(FDateTime::MinValue())
		, AccessTimeStamp(FDateTime::MinValue())
		, Size(-1)
	{
	}
};

/**
 * A helper class for wrapping some of the network file payload specifics
 */
class FNetworkFileArchive : public FBufferArchive
{
public:
	FNetworkFileArchive(uint32 Command)
	{
		// make sure the command is at the start
		*this << Command;
	}

	// helper to serialize TCHAR* (there are a lot)
	inline friend FNetworkFileArchive& operator<<(FNetworkFileArchive& Ar, const TCHAR*& Str)
	{
		FString Temp(Str);
		Ar << Temp;
		return Ar;
	}
};
