// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/DateTime.h"

/**
 * Helper struct to hold and quickly access server TOC.
 */
struct FServerTOC
{
	/** List of files in a directory. */
	typedef TMap<FString, FDateTime> FDirectory;

	/** This is the "TOC" of the server */
	TMap<FString, FDirectory*> Directories;

	/** Destructor. Destroys directories. */
	SOCKETS_API ~FServerTOC();

	/**
	 * Adds a file or directory to TOC.
	 *
	 * @param Filename File name or directory name to add.
	 * @param Timestamp File timestamp. Directories should have this set to 0.
	 */
	SOCKETS_API void AddFileOrDirectory(const FString& Filename, const FDateTime& Timestamp);

	/**
	 * Finds a file in TOC.
	 *
	 * @param Filename File name to find.
	 * @return Pointer to a timestamp if the file was found, NULL otherwise.
	 */
	SOCKETS_API const FDateTime* FindFile(const FString& Filename) const;

	/**
	 * Finds a directory in TOC.
	 *
	 * @param Directory Directory to find.
	 * @return Pointer to a FDirectory if the directory was found, NULL otherwise.
	 */
	SOCKETS_API const FDirectory* FindDirectory(const FString& Directory) const;

	/**
	 * Finds a directory in TOC non const version used internally 
	 * see FindDirectory
	 *
	 * @param Directory Directory to find
	 * @return pointer to a FDirectory if the directory was found, null otherwise
	 */
	SOCKETS_API FDirectory* FindDirectory(const FString& Directory);


	SOCKETS_API int32 RemoveFileOrDirectory(const FString& Filename);
};
