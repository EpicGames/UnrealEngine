// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AddressInfoTypes.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StringConv.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "IPAddress.h"
#include "Logging/LogMacros.h"
#include "Net/Common/Sockets/SocketErrors.h"
#include "SocketTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class Error;
class FInternetAddr;
class FOutputDevice;
class FSocket;
struct FPacketTimestamp;

SOCKETS_API DECLARE_LOG_CATEGORY_EXTERN(LogSockets, Log, All);

// Need to guarantee the "default" socket subsystem on these platforms
// as other subsystems (ie Steam) might override the default
#ifndef PLATFORM_SOCKETSUBSYSTEM
	#if PLATFORM_WINDOWS
		#define PLATFORM_SOCKETSUBSYSTEM FName("WINDOWS")
	#elif PLATFORM_MAC
		#define PLATFORM_SOCKETSUBSYSTEM FName("MAC")
	#elif PLATFORM_IOS
		#define PLATFORM_SOCKETSUBSYSTEM FName("IOS")
	#elif PLATFORM_UNIX
		#define PLATFORM_SOCKETSUBSYSTEM FName("UNIX")
	#elif PLATFORM_ANDROID
		#define PLATFORM_SOCKETSUBSYSTEM FName("ANDROID")
	#else
		#define PLATFORM_SOCKETSUBSYSTEM FName()
	#endif
#endif

/** A unique pointer that will automatically call ISocketSubsystem::DestroySocket on its owned socket. */
class FSocketDeleter;

using FUniqueSocket = TUniquePtr<FSocket, FSocketDeleter>;

/**
 * This is the base interface to abstract platform specific sockets API
 * differences.
 */
class ISocketSubsystem
{
public:

	/**
	 * Get the singleton socket subsystem for the given named subsystem
	 */
	static SOCKETS_API ISocketSubsystem* Get(const FName& SubsystemName=NAME_None);

	/**
	 * Shutdown all registered subsystems
	 */
	static SOCKETS_API void ShutdownAllSystems();


	SOCKETS_API ISocketSubsystem();
	SOCKETS_API virtual ~ISocketSubsystem();

	/**
	 * Does per platform initialization of the sockets library
	 *
	 * @param Error a string that is filled with error information
	 *
	 * @return true if initialized ok, false otherwise
	 */
	virtual bool Init(FString& Error) = 0;

	/**
	 * Performs platform specific socket clean up
	 */
	virtual void Shutdown() = 0;

	/**
	 * Creates a socket
	 *
	 * @Param SocketType type of socket to create (DGram, Stream, etc)
	 * @param SocketDescription debug description
	 * @param bForceUDP overrides any platform specific protocol with UDP instead
	 *
	 * @return the new socket or NULL if failed
	 */
	virtual FSocket* CreateSocket(const FName& SocketType, const FString& SocketDescription, bool bForceUDP = false)
	{
		const FName NoProtocolTypeName(NAME_None);
		return CreateSocket(SocketType, SocketDescription, NoProtocolTypeName);
	}

	/**
	 * Creates a socket
	 *
	 * @Param SocketType type of socket to create (DGram, Stream, etc)
	 * @param SocketDescription debug description
	 * @param ProtocolType the socket protocol to be used. Each subsystem must handle the None case and output a valid socket regardless.
	 *
	 * @return the new socket or NULL if failed
	 */
	UE_DEPRECATED(4.23, "Use the CreateSocket with the FName parameter for support for multiple protocol types.")
	virtual FSocket* CreateSocket(const FName& SocketType, const FString& SocketDescription, ESocketProtocolFamily ProtocolType)
	{
		return CreateSocket(SocketType, SocketDescription, GetProtocolNameFromFamily(ProtocolType));
	}

	/**
	 * Creates a socket using the given protocol name.
	 *
	 * @Param SocketType type of socket to create (DGram, Stream, etc)
	 * @param SocketDescription debug description
	 * @param ProtocolName the name of the internet protocol to use for this socket. None should be handled.
	 *
	 * @return the new socket or NULL if failed
	 */
	virtual FSocket* CreateSocket(const FName& SocketType, const FString& SocketDescription, const FName& ProtocolName) = 0;

	/**
	 * Creates a socket wrapped in a unique pointer that will call DestroySocket automatically - do not call it explicitly!
	 * This SocketSubsystem must also outlive the sockets it creates.
	 *
	 * @Param SocketType type of socket to create (DGram, Stream, etc)
	 * @param SocketDescription debug description
	 * @param bForceUDP overrides any platform specific protocol with UDP instead
	 *
	 * @return the new socket or NULL if failed
	 */
	SOCKETS_API FUniqueSocket CreateUniqueSocket(const FName& SocketType, const FString& SocketDescription, bool bForceUDP = false);

	/**
	 * Creates a socket using the given protocol name,  wrapped in a unique pointer that will call DestroySocket automatically - do not call it explicitly!
	 * This SocketSubsystem must also outlive the sockets it creates.
	 *
	 * @Param SocketType type of socket to create (DGram, Stream, etc)
	 * @param SocketDescription debug description
	 * @param ProtocolName the name of the internet protocol to use for this socket. None should be handled.
	 *
	 * @return the new socket or NULL if failed
	 */
	SOCKETS_API FUniqueSocket CreateUniqueSocket(const FName& SocketType, const FString& SocketDescription, const FName& ProtocolName);

	/**
	 * Creates a resolve info cached struct to hold the resolved address
	 *
	 * @Param Addr address to resolve for the socket subsystem
	 *
	 * @return the new resolved address or NULL if failed
	 */
	SOCKETS_API virtual class FResolveInfoCached* CreateResolveInfoCached(TSharedPtr<FInternetAddr> Addr) const;

	/**
	 * Cleans up a socket class
	 *
	 * @param Socket the socket object to destroy
	 */
	virtual void DestroySocket(FSocket* Socket) = 0;

	/**
	 * Gets the address information of the given hostname and outputs it into an array of resolvable addresses.
	 * It is up to the caller to determine which one is valid for their environment.
	 *
	 * @param HostName string version of the queryable hostname or ip address
	 * @param ServiceName string version of a service name ("http") or a port number ("80")
	 * @param QueryFlags What flags are used in making the getaddrinfo call. Several flags can be used at once by
	 *                   bitwise OR-ing the flags together.
	 *                   Platforms are required to translate this value into a the correct flag representation.
	 * @param ProtocolType Used to limit results from the call. Specifying None will search all valid protocols.
	 *					   Callers will find they rarely have to specify this flag.
	 * @param SocketType What socket type should the results be formatted for. This typically does not change any
	 *                   formatting results and can be safely left to the default value.
	 *
	 * @return the results structure containing the array of address results to this query
	 */
	UE_DEPRECATED(4.23, "Migrate to GetAddressInfo that takes an FName as the protocol specification.")
	virtual FAddressInfoResult GetAddressInfo(const TCHAR* HostName, const TCHAR* ServiceName = nullptr,
		EAddressInfoFlags QueryFlags = EAddressInfoFlags::Default,
		ESocketProtocolFamily ProtocolType = ESocketProtocolFamily::None,
		ESocketType SocketType = ESocketType::SOCKTYPE_Unknown)
	{
		return GetAddressInfo(HostName, ServiceName, QueryFlags, GetProtocolNameFromFamily(ProtocolType), SocketType);
	}

	/**
	 * Gets the address information of the given hostname and outputs it into an array of resolvable addresses.
	 * It is up to the caller to determine which one is valid for their environment.
	 *
	 * This function allows for specifying FNames for the protocol type, allowing for support of other 
	 * platform protocols
	 *
	 * @param HostName string version of the queryable hostname or ip address
	 * @param ServiceName string version of a service name ("http") or a port number ("80")
	 * @param QueryFlags What flags are used in making the getaddrinfo call. Several flags can be used at once by
	 *                   bitwise OR-ing the flags together.
	 *                   Platforms are required to translate this value into a the correct flag representation.
	 * @param ProtocolTypeName Used to limit results from the call. Specifying None will search all valid protocols.
	 *					   Callers will find they rarely have to specify this flag.
	 * @param SocketType What socket type should the results be formatted for. This typically does not change any
	 *                   formatting results and can be safely left to the default value.
	 *
	 * @return the results structure containing the array of address results to this query
	 */
	virtual FAddressInfoResult GetAddressInfo(const TCHAR* HostName, const TCHAR* ServiceName = nullptr,
		EAddressInfoFlags QueryFlags = EAddressInfoFlags::Default,
		const FName ProtocolTypeName = NAME_None,
		ESocketType SocketType = ESocketType::SOCKTYPE_Unknown) = 0;

	/**
	 * Async variant of GetAddressInfo that fetches the data from the above function in an 
	 * asynchronous task executed on an available background thread.
	 * 
	 * On Completion, this fires a callback function that will be executed on the same thread as
	 * the task's execution. The caller is responsible for either dispatching the result to the thread of their choosing
	 * or to allow the result callback execute on the task's thread.
	 *
	 * This function allows for specifying FNames for the protocol type, allowing for support of other
	 * platform protocols
	 *
	 * @param Callback the callback function to fire when this query completes. Contains the FAddressInfoResult structure.
	 * @param HostName string version of the queryable hostname or ip address
	 * @param ServiceName string version of a service name ("http") or a port number ("80")
	 * @param QueryFlags What flags are used in making the getaddrinfo call. Several flags can be used at once by
	 *                   bitwise OR-ing the flags together.
	 *                   Platforms are required to translate this value into a the correct flag representation.
	 * @param ProtocolTypeName Used to limit results from the call. Specifying None will search all valid protocols.
	 *					   Callers will find they rarely have to specify this flag.
	 * @param SocketType What socket type should the results be formatted for. This typically does not change any
	 *                   formatting results and can be safely left to the default value.
	 *
	 */
	SOCKETS_API virtual void GetAddressInfoAsync(FAsyncGetAddressInfoCallback Callback, const TCHAR* HostName,
		const TCHAR* ServiceName = nullptr, EAddressInfoFlags QueryFlags = EAddressInfoFlags::Default,
		const FName ProtocolTypeName = NAME_None,
		ESocketType SocketType = ESocketType::SOCKTYPE_Unknown);

	/**
	 * Serializes a string that only contains an address.
	 * 
	 * This is a what you see is what you get, there is no DNS resolution of the input string,
	 * so only use this if you know you already have a valid address and will not need to convert.
	 * Otherwise, feed the address to GetAddressInfo for guaranteed results.
	 *
	 * @param InAddress the address to serialize
	 *
	 * @return The FInternetAddr of the given string address. This will point to nullptr on failure.
	 */
	virtual TSharedPtr<FInternetAddr> GetAddressFromString(const FString& InAddress) = 0;

	/**
	 * Does a DNS look up of a host name
	 * This code assumes a lot, and as such, it's not guaranteed that the results provided by it are correct.
	 *
	 * @param HostName the name of the host to look up
	 * @param Addr the address to copy the IP address to
	 */
	UE_DEPRECATED(4.23, "Please use GetAddressInfo to query hostnames")
	virtual ESocketErrors GetHostByName(const ANSICHAR* HostName, FInternetAddr& OutAddr)
	{
		FAddressInfoResult GAIResult = GetAddressInfo(ANSI_TO_TCHAR(HostName), nullptr, EAddressInfoFlags::Default, NAME_None);
		if (GAIResult.Results.Num() > 0)
		{
			OutAddr.SetRawIp(GAIResult.Results[0].Address->GetRawIp());
			return SE_NO_ERROR;
		}

		return SE_HOST_NOT_FOUND;
	}

	/**
	 * Creates a platform specific async hostname resolution object
	 *
	 * @param HostName the name of the host to look up
	 *
	 * @return the resolve info to query for the address
	 */
	SOCKETS_API virtual class FResolveInfo* GetHostByName(const ANSICHAR* HostName);

	/**
	 * Some platforms require chat data (voice, text, etc.) to be placed into
	 * packets in a special way. This function tells the net connection
	 * whether this is required for this platform
	 */
	virtual bool RequiresChatDataBeSeparate() = 0;

	/**
	 * Some platforms require packets be encrypted. This function tells the
	 * net connection whether this is required for this platform
	 */
	virtual bool RequiresEncryptedPackets() = 0;

	/**
	 * Determines the name of the local machine
	 *
	 * @param HostName the string that receives the data
	 *
	 * @return true if successful, false otherwise
	 */
	virtual bool GetHostName(FString& HostName) = 0;

	/**
	 * Create a proper FInternetAddr representation
	 * @param Address host address
	 * @param Port host port
	 */
	UE_DEPRECATED(4.23, "To support different address sizes, use CreateInternetAddr with no arguments and call SetIp/SetRawIp and SetPort on the returned object")
	virtual TSharedRef<FInternetAddr> CreateInternetAddr(uint32 Address, uint32 Port = 0)
	{
		TSharedRef<FInternetAddr> ReturnAddr = CreateInternetAddr();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ReturnAddr->SetIp(Address);
		ReturnAddr->SetPort(Port);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return ReturnAddr;
	}

	/**
	 * Create a proper FInternetAddr representation
	 */
	virtual TSharedRef<FInternetAddr> CreateInternetAddr() = 0;

	/**
	 * Create a FInternetAddr of the desired protocol
	 */
	virtual TSharedRef<FInternetAddr> CreateInternetAddr(const FName ProtocolType)
	{
		// If not implemented, returns the base version
		return CreateInternetAddr();
	}

	/**
	 * Create a platform specific FRecvMulti representation
	 *
	 * @param MaxNumPackets			The maximum number of packet receives supported
	 * @param MaxPacketSize			The maximum supported packet size
	 * @param Flags					Flags for specifying how FRecvMulti should be initialized (for e.g. retrieving timestamps)
	 * @return						Returns the platform specific FRecvMulti instance
	 */
	SOCKETS_API virtual TUniquePtr<FRecvMulti> CreateRecvMulti(int32 MaxNumPackets, int32 MaxPacketSize,
													ERecvMultiFlags Flags=ERecvMultiFlags::None);

	/**
	 * @return Whether the machine has a properly configured network device or not
	 */
	virtual bool HasNetworkDevice() = 0;

	/**
	 *	Get the name of the socket subsystem
	 * @return a string naming this subsystem
	 */
	virtual const TCHAR* GetSocketAPIName() const = 0;

	/**
	 * Returns the last error that has happened
	 */
	virtual ESocketErrors GetLastErrorCode() = 0;

	/**
	 * Translates the platform error code to a ESocketErrors enum
	 */
	virtual ESocketErrors TranslateErrorCode(int32 Code) = 0;


	// The following functions are not expected to be overridden

	/**
	 * Returns a human readable string from an error code
	 *
	 * @param Code the error code to check
	 */
	SOCKETS_API const TCHAR* GetSocketError(ESocketErrors Code = SE_GET_LAST_ERROR_CODE);

	/**
	 * Gets the list of addresses associated with the adapters on the local computer.
	 * Unlike GetLocalHostAddr, this function does not give preferential treatment to multihome options in results.
	 * It's encouraged that users check for multihome before using the results of this function.
	 *
	 * @param OutAddresses - Will hold the address list.
	 *
	 * @return true on success, false otherwise.
	 */
	SOCKETS_API virtual bool GetLocalAdapterAddresses(TArray<TSharedPtr<FInternetAddr>>& OutAddresses);

	/**
	 * Get a local IP to bind to.
	 *
	 * Typically, it is better to use GetLocalBindAddresses as it better supports hybrid network functionality
	 * and less chances for connections to fail due to mismatched protocols.
	 */
	SOCKETS_API virtual TSharedRef<FInternetAddr> GetLocalBindAddr(FOutputDevice& Out);

	/**
	 * Get bindable addresses that this machine can use as reported by GetAddressInfo with the BindableAddress flag.
	 * This will return the various any address for usage. If multihome has been specified, only the multihome address
	 * will be returned in the array.
	 *
	 * @return If GetAddressInfo succeeded or multihome is specified, an array of addresses that can be binded on. Failure returns an empty array.
	 */
	SOCKETS_API virtual TArray<TSharedRef<FInternetAddr>> GetLocalBindAddresses();

	/**
	 * Bind to next available port.
	 *
	 * @param Socket The socket that that will bind to the port
	 * @param Addr The local address and port that is being bound to (usually the result of GetLocalBindAddr()). This addresses port number will be modified in place
	 * @param PortCount How many ports to try
	 * @param PortIncrement The amount to increase the port number by on each attempt
	 *
	 * @return The bound port number, or 0 on failure
	 */
	SOCKETS_API int32 BindNextPort(FSocket* Socket, FInternetAddr& Addr, int32 PortCount, int32 PortIncrement);

	/**
	 * Uses the platform specific look up to determine the host address
	 *
	 * To better support multiple network interfaces and remove ambiguity between address protocols, 
	 * it is encouraged to use GetLocalAdapterAddresses to determine machine addresses. 
	 * Be sure to check GetMultihomeAddress ahead of time.
	 *
	 * @param Out the output device to log messages to
	 * @param bCanBindAll true if all can be bound (no primarynet), false otherwise
	 *
	 * @return The local host address
	 */
	SOCKETS_API virtual TSharedRef<FInternetAddr> GetLocalHostAddr(FOutputDevice& Out, bool& bCanBindAll);

	/**
	 * Returns the multihome address if the flag is present and valid. For ease of use, 
	 * this function will check validity of the address for the caller.
	 * 
	 * @param Addr the address structure which will have the Multihome address in it if set.
	 *
	 * @return If the multihome address was set and valid
	 */
	SOCKETS_API virtual bool GetMultihomeAddress(TSharedRef<FInternetAddr>& Addr);

	/**
	 * Checks the host name cache for an existing entry (faster than resolving again)
	 *
	 * @param HostName the host name to search for
	 * @param Addr the out param that the IP will be copied to
	 *
	 * @return true if the host was found, false otherwise
	 */
	SOCKETS_API bool GetHostByNameFromCache(const ANSICHAR* HostName, TSharedPtr<class FInternetAddr>& Addr);

	/**
	 * Stores the ip address with the matching host name
	 *
	 * @param HostName the host name to search for
	 * @param Addr the IP that will be copied from
	 */
	SOCKETS_API void AddHostNameToCache(const ANSICHAR* HostName, TSharedPtr<class FInternetAddr> Addr);

	/**
	 * Removes the host name to ip mapping from the cache
	 *
	 * @param HostName the host name to search for
	 */
	SOCKETS_API void RemoveHostNameFromCache(const ANSICHAR* HostName);

	/**
	 * Returns true if FSocket::RecvMulti is supported by this socket subsystem
	 */
	SOCKETS_API virtual bool IsSocketRecvMultiSupported() const;


	/**
	 * Returns true if FSocket::Wait and QuerySocketStateMany is supported by this socket subsystem.
	 */
	virtual bool IsSocketWaitSupported() const = 0;


	/**
	 * Converts a platform packet timestamp, into a local timestamp, or into a time delta etc.
	 *
	 * @param Timestamp		The timestamp to translate
	 * @param Translation	The type of translation to perform on the timestamp (time delta is usually faster than local timestamp)
	 * @return				Returns the translated timestamp or delta
	 */
	SOCKETS_API virtual double TranslatePacketTimestamp(const FPacketTimestamp& Timestamp,
											ETimestampTranslation Translation=ETimestampTranslation::LocalTimestamp);

	/**
	 * Returns true if FSocket::RecvFromWithPktInfo is supported by this socket subsystem.
	 */
	SOCKETS_API virtual bool IsRecvFromWithPktInfoSupported() const;

	enum class ESocketStateFlags : uint8
	{
		Readable = 1 << 0,
		Writable = 1 << 1,
		Error = 1 << 2,
	};

	/**
	 * Queries the state of multiple sockets in a single call (equivalent to poll/select).
	 * Not supported on all platforms: check IsSocketWaitSupported() before calling.
	 *
	 * @param Sockets            The sockets to query.
	 * @param InOutSocketStates  On input, the flags to query per socket (Readable, Writable, Error).
	 *                           On output, the flags that are actually set. Must be the same length as Sockets.
	 * @param WaitTime           Maximum time to block waiting for any socket to become ready.
	 *                           Pass FTimespan(-1) to block indefinitely, or FTimespan::Zero() to poll without blocking.
	 */
	SOCKETS_API virtual void QuerySocketStateMany(TArrayView<FSocket*> Sockets, TArray<ESocketStateFlags>& InOutSocketStates, FTimespan WaitTime) const;

protected:

	/**
	 * Conversion functions from the SocketProtocolFamily enum to the new FName system.
	 * For now, both are supported, but it's better to use the FName when possible.
	 */
	SOCKETS_API virtual ESocketProtocolFamily GetProtocolFamilyFromName(const FName& InProtocolName) const;
	SOCKETS_API virtual FName GetProtocolNameFromFamily(ESocketProtocolFamily InProtocolFamily) const;

private:

	/** Used to provide threadsafe access to the host name cache */
	FCriticalSection HostNameCacheSync;

	/** Stores a resolved IP address for a given host name */
	TMap<FString, TSharedPtr<FInternetAddr> > HostNameCache;
};

/** Deleter object that can be used with unique & shared pointers that store FSockets. The SocketSubsystem must be valid when the call operator is invoked! */
class FSocketDeleter
{
public:
	FSocketDeleter()
		: Subsystem(nullptr)
	{
	}

	FSocketDeleter(ISocketSubsystem* InSubsystem)
		: Subsystem(InSubsystem)
	{
	}

	void operator()(FSocket* Socket) const
	{
		if (Subsystem && Socket)
		{
			Subsystem->DestroySocket(Socket);
		}
	}

private:
	ISocketSubsystem* Subsystem;
};

typedef TSharedPtr<ISocketSubsystem> IOnlineSocketPtr;


