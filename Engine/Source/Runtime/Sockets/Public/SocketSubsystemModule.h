// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

/**
 * Socket subsystem module class
 * Wraps the loading of an socket subsystem by name and allows new services to register themselves for use
 */
class FSocketSubsystemModule : public IModuleInterface
{
private:

	/** Name of the default socket subsystem defined by the platform */
	FName DefaultSocketSubsystem;

	/** Mapping of all currently loaded subsystems to their name */
	TMap<FName, class ISocketSubsystem*> SocketSubsystems;

public:

	FSocketSubsystemModule() {}
	virtual ~FSocketSubsystemModule() = default;

	/** 
	 * Main entry point for accessing a socket subsystem by name
	 * Will load the appropriate module if the subsystem isn't currently loaded
	 * It's possible that the subsystem doesn't exist and therefore can return NULL
	 *
	 * @param SubsystemName - name of subsystem as referenced by consumers
	 * @return Requested socket subsystem, or NULL if that subsystem was unable to load or doesn't exist
	 */
	virtual class ISocketSubsystem* GetSocketSubsystem(const FName InSubsystemName = NAME_None);

	/** 
	 * Register a new socket subsystem interface with the base level factory provider
	 * @param FactoryName - name of subsystem as referenced by consumers
	 * @param Factory - instantiation of the socket subsystem interface, this will take ownership
	 * @param bMakeDefault - make this subsystem the default
	 */
	virtual void RegisterSocketSubsystem(const FName FactoryName, class ISocketSubsystem* Factory, bool bMakeDefault=false);
	
	/** 
	 * Unregister an existing online subsystem interface from the base level factory provider
	 * @param FactoryName - name of subsystem as referenced by consumers
	 */
	virtual void UnregisterSocketSubsystem(const FName FactoryName);


	// IModuleInterface

	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 * Overloaded to allow the default subsystem a chance to load
	 */
	virtual void StartupModule() override;

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 * Overloaded to shut down all loaded online subsystems
	 */
	virtual void ShutdownModule() override;

	/**
	 * Override this to set whether your module is allowed to be unloaded on the fly
	 *
	 * @return	Whether the module supports shutdown separate from the rest of the engine.
	 */
	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

	/**
	 * Override this to set whether your module would like cleanup on application shutdown
	 *
	 * @return	Whether the module supports shutdown on application exit
	 */
	virtual bool SupportsAutomaticShutdown() override
	{
		return false;
	}
};

/** Public references to the socket subsystem module pointer should use this */
typedef TSharedPtr<FSocketSubsystemModule> FSocketSubsystemModulePtr;

