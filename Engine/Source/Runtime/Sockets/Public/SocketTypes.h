// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Net/Common/Packets/PacketView.h"

// Forward declarations
class FInternetAddr;
class ISocketSubsystem;
class FSocket;


/** Indicates the socket protocol of socket being used, typically for BSD Sockets */
enum class ESocketProtocolFamily : uint8
{
	/** No protocol family specification. Typically defined as AF_UNSPEC */
	None,
	/** IPv4 and IPv6 respectively. */
	IPv4,
	IPv6
};

/** Used for indicating the socket network protocol to be used in addressing and socket creation */
namespace FNetworkProtocolTypes
{
	SOCKETS_API extern const FLazyName IPv4;
	SOCKETS_API extern const FLazyName IPv6;
}

/** Indicates the type of socket being used (streaming or datagram) */
enum ESocketType
{
	/** Not bound to a protocol yet */
	SOCKTYPE_Unknown,
	/** A UDP type socket */
	SOCKTYPE_Datagram,
	/** A TCP type socket */
	SOCKTYPE_Streaming
};

/** Indicates the connection state of the socket */
enum ESocketConnectionState
{
	SCS_NotConnected,
	SCS_Connected,
	/** Indicates that the end point refused the connection or couldn't be reached */
	SCS_ConnectionError
};


namespace ESocketReceiveFlags
{
	/**
	 * Enumerates socket receive flags.
	 */
	enum Type
	{
		/**
		 * Return as much data as is currently available in the input queue,
		 * up to the specified size of the receive buffer.
		 */
		None = 0,

		/**
		 * Copy received data into the buffer without removing it from the input queue.
		 */
		Peek = 2,

		/**
		 * Block the receive call until either the supplied buffer is full, the connection
		 * has been closed, the request has been canceled, or an error occurred.
		 */
		WaitAll = 0x100
	};
}


namespace ESocketWaitConditions
{
	/**
	 * Enumerates socket wait conditions.
	 */
	enum Type
	{
		/**
		 * Wait until data is available for reading.
		 */
		WaitForRead,

		/**
		 * Wait until data can be written.
		 */
		WaitForWrite,

		/**
		 * Wait until data is available for reading or can be written.
		 */
		WaitForReadOrWrite
	};
}

/**
 * Enumerates socket shutdown modes.
 */
enum class ESocketShutdownMode
{
	/**
	 * Disables reading on the socket.
	 */
	Read,
	
	/**
	 * Disables writing on the socket.
	 */
	Write,
	
	/**
	 * Disables reading and writing on the socket.
	 */
	ReadWrite
};




/**
 * Specifies how a platform specific timestamp (in this case, a packet timestamp) should be translated
 */
enum class ETimestampTranslation : uint8
{
	/**
	 * Translates the timestamp into a local timestamp, comparable (with accuracy caveats) to other local timestamps.
	 *
	 * Use this to get a value comparable to FPlatformTime::Seconds() - if the platform timestamp uses a different clock,
	 * this has both a performance (2x internal clock reads i.e. 2x FPlatformTime::Seconds() calls) and accuracy caveat.
	 */
	LocalTimestamp,

	/**
	 * The delta between present platform time vs timestamp - faster than local translation, less accuracy caveats.
	 *
	 * This is the most accurate measure of time passed since the packet was recorded at an OS/NIC/Thread level,
	 * to TranslatePacketTimestamp being called for the packets timestamp - and performs faster (1x FPlatformTime::Seconds() call).
	 */
	TimeDelta
};

/**
 * Flags for specifying how an FRecvMulti instance should be initialized
 */
enum class ERecvMultiFlags : uint32
{
	None				= 0x00000000,
	RetrieveTimestamps	= 0x00000001	// Whether or not to support retrieving timestamps
};

ENUM_CLASS_FLAGS(ERecvMultiFlags);


/**
 * Stores the persistent state and packet buffers/data, for receiving packets with FSocket::RecvMulti.
 * To optimize performance, use only once instance of this struct, for the lifetime of the socket.
 */
struct FRecvMulti : public FNoncopyable, public FVirtualDestructor
{
	friend struct FUnixRecvMulti;
	friend class FSocketUnix;

private:
	/**
	 * Receive data for each individual packet
	 */
	struct FRecvData
	{
		/** The source address for the packet */
		TSharedPtr<FInternetAddr>	Source;

		/** Pointer to the packet data */
		const uint8*				Data;

		/** Internal pointer specifying the number of bytes read */
		const uint32*				BytesReadPtr;


		FRecvData()
			: Source()
			, Data(nullptr)
			, BytesReadPtr(nullptr)
		{
		}
	};


private:
	/** The current list of received packets */
	TUniquePtr<FRecvData[]>			Packets;

	/** The number of packets received */
	int32							NumPackets;

public:
	/** The maximum number of packets this FRecvMulti instance can support */
	const int32						MaxNumPackets;

	/** The maximum packet size this FRecvMulti instance can support */
	const int32						MaxPacketSize;


private:
	/**
	 * Initialize an FRecvMulti instance, supporting the specified maximum packet count/sizes
	 *
	 * @param SocketSubsystem		The socket subsystem initializing this FRecvMulti instance
	 * @param InMaxNumPackets		The maximum number of packet receives supported
	 * @param InMaxPacketSize		The maximum supported packet size
	 * @param bRetrieveTimestamps	Whether or not to support retrieving timestamps
	 */
	SOCKETS_API FRecvMulti(ISocketSubsystem* SocketSubsystem, int32 InMaxNumPackets, int32 InMaxPacketSize,
				ERecvMultiFlags InitFlags=ERecvMultiFlags::None);


public:
	/**
	 * Retrieves the information for the specified packet
	 *
	 * @param PacketIdx		The RecvMulti index for the packet to be retrieved
	 * @param OutPacket		Outputs a view to the received packet
	 */
	void GetPacket(int32 PacketIdx, FReceivedPacketView& OutPacket)
	{
		check(PacketIdx >= 0);
		check(PacketIdx < NumPackets);

		FRecvData& CurData = Packets.Get()[PacketIdx];

		OutPacket.DataView = {CurData.Data, (int32)*CurData.BytesReadPtr, ECountUnits::Bytes};
		OutPacket.Address = CurData.Source;
		OutPacket.Error = ESocketErrors::SE_NO_ERROR;
	}

	/**
	 * Returns the platform specific timestamp for when the specified packet was received by the operating system
	 *
	 * @param PacketIdx		The index into Packets, of the packet to be checked
	 * @param OutTimestamp	The timestamp for the specified packet
	 * @return				Whether or not the timestamp was retrieved successfully
	 */
	virtual bool GetPacketTimestamp(int32 PacketIdx, FPacketTimestamp& OutTimestamp) const = 0;

	/**
	 * Retrieves the current number of received packets
	 */
	int32 GetNumPackets() const
	{
		return NumPackets;
	}


	/**
	 * Calculates the total memory consumption of this FRecvMulti instance, including platform-specific data
	 *
	 * @param Ar	The archive being used to count the memory consumption
	 */
	SOCKETS_API virtual void CountBytes(FArchive& Ar) const;
};
