// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Misc/Timespan.h"
#include "SocketTypes.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FInternetAddr;

/**
 * This is our abstract base class that hides the platform specific socket implementation
 */
class FSocket : public TSharedFromThis<FSocket, ESPMode::ThreadSafe>
{
protected:

	/** Indicates the type of socket this is */
	const ESocketType SocketType;

	/** Debug description of socket usage. */
	FString SocketDescription;

	/** Protocol used in creation of a socket */
	FName SocketProtocol;

public:

	/** Default constructor. */
	SOCKETS_API FSocket();

	/**
	 * Specifies the type of socket being created
	 *
	 * @param InSocketType The type of socket being created
	 * @param InSocketDescription The debug description of the socket
	 */
	UE_DEPRECATED(4.22, "Please migrate to the constructor that specifies protocol stack")
	inline FSocket(ESocketType InSocketType, const FString& InSocketDescription) :
		SocketType(InSocketType),
		SocketDescription(InSocketDescription),
		SocketProtocol(NAME_None)
	{ }

	UE_DEPRECATED(4.23, "Please migrate to the constructor that takes an FName for the protocol stack")
		inline FSocket(ESocketType InSocketType, const FString& InSocketDescription, ESocketProtocolFamily InSocketProtocol) :
		SocketType(InSocketType),
		SocketDescription(InSocketDescription)
	{ 
		if (InSocketProtocol == ESocketProtocolFamily::IPv4)
		{
			SocketProtocol = FNetworkProtocolTypes::IPv4;
		}
		else if (InSocketProtocol == ESocketProtocolFamily::IPv6)
		{
			SocketProtocol = FNetworkProtocolTypes::IPv6;
		}
		else
		{
			SocketProtocol = NAME_None;
		}
	}

	/**
	 * Specifies the type of socket being created
	 *
	 * @param InSocketType The type of socket being created
	 * @param InSocketDescription The debug description of the socket
	 * @param InSocketProtocol the protocol stack this socket should be created on.
	 */
	SOCKETS_API FSocket(ESocketType InSocketType, const FString& InSocketDescription, const FName& InSocketProtocol);

	/** Virtual destructor. */
	SOCKETS_API virtual ~FSocket();

	/**
	 * Shuts down the socket, making it unusable for reads and/or writes. This does not close the socket!
	 *
	 * @param Types
	 * @return true if successful, false otherwise.
	 */
	virtual bool Shutdown(ESocketShutdownMode Mode) = 0;

	/**
	 * Closes the socket
	 *
	 * @return true if it closes without errors, false otherwise.
	 */
	virtual bool Close() = 0;

	/**
	 * Releases the underlying socket implementation for use with native APIs.
	 * The user of this class must know the underlying implementation details in order to cast the return value properly.
	 * If this function succeeds, the FSocket instance no longer controls the connection (it will not Close()).
	 * @return a non-zero value upon success, or zero if not implemented/failure to release ownership.
	 */
	[[nodiscard]] virtual UPTRINT ReleaseNativeSocket() { return 0; }

	/**
	 * Binds a socket to a network byte ordered address.
	 *
	 * @param Addr The address to bind to.
	 * @return true if successful, false otherwise.
	 */
	virtual bool Bind(const FInternetAddr& Addr) = 0;

	/**
	 * Connects a socket to a network byte ordered address.
	 *
	 * @param Addr The address to connect to.
	 * @return true if successful, false otherwise.
	 */
	virtual bool Connect(const FInternetAddr& Addr) = 0;

	/**
	 * Places the socket into a state to listen for incoming connections.
	 *
	 * @param MaxBacklog The number of connections to queue before refusing them.
	 * @return true if successful, false otherwise.
	 */
	virtual bool Listen(int32 MaxBacklog) = 0;

	/**
	 * Waits for a pending connection on the socket.
	 *
	 * @param out bHasPendingConnection Will indicate whether a connection is pending or not.
	 * @param WaitTime The maximum time to wait for a connection. If zero, the function will not 
	 *                 wait and will return immediately with a value in bHasPendingConnection.
	 * @return true if successful, false otherwise.
	 */
	virtual bool WaitForPendingConnection(bool& bHasPendingConnection, const FTimespan& WaitTime) = 0;

	/**
	 * Queries the socket to determine if there is a pending connection.
	 *
	 * @param bHasPendingConnection Will indicate whether a connection is pending or not.
	 * @return true if successful, false otherwise.
	 */
	bool HasPendingConnection(bool& bHasPendingConnection)
	{
		return WaitForPendingConnection(bHasPendingConnection, FTimespan::Zero());
	}

	/**
	* Queries the socket to determine if there is pending data on the queue.
	*
	* @param PendingDataSize Will indicate how much data is on the pipe for a single recv call.
	* @return true if the socket has data, false otherwise.
	*/
	virtual bool HasPendingData(uint32& PendingDataSize) = 0;

	/**
	 * Accepts a connection that is pending.
	 *
	 * @param SocketDescription Debug description of socket,
	 * @return The new (heap-allocated) socket, or nullptr if unsuccessful.
	 */
	virtual class FSocket* Accept(const FString& InSocketDescription) = 0;

	/**
	 * Accepts a connection that is pending.
	 *
	 * @param OutAddr The address of the connection.
	 * @param SocketDescription Debug description of socket.
	 * @return The new (heap-allocated) socket, or nullptr if unsuccessful.
	 */
	virtual class FSocket* Accept(FInternetAddr& OutAddr, const FString& InSocketDescription) = 0;

	/**
	 * Sends a buffer to a network byte ordered address.
	 *
	 * @param Data The buffer to send.
	 * @param Count The size of the data to send.
	 * @param BytesSent Will indicate how much was sent.
	 * @param Destination The network byte ordered address to send to.
	 */
	SOCKETS_API virtual bool SendTo(const uint8* Data, int32 Count, int32& BytesSent, const FInternetAddr& Destination);

	/**
	 * Sends a buffer on a connected socket.
	 *
	 * @param Data The buffer to send.
	 * @param Count The size of the data to send.
	 * @param BytesSent Will indicate how much was sent.
	 */
	SOCKETS_API virtual bool Send(const uint8* Data, int32 Count, int32& BytesSent);

	/**
	 * Reads a chunk of data from the socket and gathers the source address.
	 *
	 * A return value of 'true' does not necessarily mean that data was returned.
	 * Callers must check the 'BytesRead' parameter for the actual amount of data
	 * returned. A value of zero indicates that there was no data available for reading.
	 *
	 * @param Data The buffer to read into.
	 * @param BufferSize The max size of the buffer.
	 * @param BytesRead Will indicate how many bytes were read from the socket.
	 * @param Source Will contain the receiving the address of the sender of the data.
	 * @param Flags The receive flags.
	 * @return true on success, false in case of a closed socket or an unrecoverable error.
	 */
	SOCKETS_API virtual bool RecvFrom(uint8* Data, int32 BufferSize, int32& BytesRead, FInternetAddr& Source, ESocketReceiveFlags::Type Flags = ESocketReceiveFlags::None);

	/**
	 * Reads a chunk of data from a connected socket
	 *
	 * A return value of 'true' does not necessarily mean that data was returned.
	 * Callers must check the 'BytesRead' parameter for the actual amount of data
	 * returned. A value of zero indicates that there was no data available for reading.
	 *
	 * @param Data The buffer to read into
	 * @param BufferSize The max size of the buffer
	 * @param BytesRead Will indicate how many bytes were read from the socket
	 * @param Flags the receive flags
	 * @return true on success, false in case of a closed socket or an unrecoverable error.
	 */
	SOCKETS_API virtual bool Recv(uint8* Data, int32 BufferSize, int32& BytesRead, ESocketReceiveFlags::Type Flags = ESocketReceiveFlags::None);

	/**
	 * Reads multiple packets from the socket at once, gathering the source address and other optional platform specific data.
	 * Use ISocketSubsystem::IsSocketRecvMultiSupported to check if the current socket platform supports this.
	 * NOTE: For optimal performance, one FRecvMulti instance should be used, for the lifetime of the socket.
	 *
	 * @param MultiData		The FRecvMulti instance that receives packet data and holds platform specific buffers for receiving data.
	 * @param Flags			The receive flags.
	 * @return				Whether or not data was successfully received
	 */
	SOCKETS_API virtual bool RecvMulti(FRecvMulti& MultiData, ESocketReceiveFlags::Type Flags=ESocketReceiveFlags::None);

	/**
	 * Blocks until the specified condition is met.
	 *
	 * @param Condition The condition to wait for.
	 * @param WaitTime The maximum time to wait.
	 * @return true if the condition was met, false if the time limit expired or an error occurred.
	 */
	virtual bool Wait(ESocketWaitConditions::Type Condition, FTimespan WaitTime) = 0;

	/**
	 * Determines the connection state of the socket.
	 *
	 * @return Connection state.
	 */
	virtual ESocketConnectionState GetConnectionState() = 0;

	/**
	 * Reads the address the socket is bound to and returns it.
	 * 
	 * @param OutAddr The address the socket is bound to.
	 */
	virtual void GetAddress(FInternetAddr& OutAddr) = 0;

	/**
	 * Reads the address of the peer the socket is connected to.
	 *
	 * @param OutAddr Address of the peer the socket is connected to.
	 * @return true if the address was retrieved correctly, false otherwise.
	 */
	virtual bool GetPeerAddress(FInternetAddr& OutAddr) = 0;

	/**
	 * Sets this socket into non-blocking mode.
	 *
	 * @param bIsNonBlocking Whether to enable blocking or not.
	 * @return true if successful, false otherwise.
	 */
	virtual bool SetNonBlocking(bool bIsNonBlocking = true) = 0;

	/**
	 * Sets a socket into broadcast mode (UDP only).
	 *
	 * @param bAllowBroadcast Whether to enable broadcast or not.
	 * @return true if successful, false otherwise.
	 */
	virtual bool SetBroadcast(bool bAllowBroadcast = true) = 0;

	/**
	 * Sets this socket into TCP_NODELAY mode (TCP only).
	 *
	 * @param bIsNoDelay Whether to enable no delay mode.
	 * @return true if successful, false otherwise.
	 */
	virtual bool SetNoDelay(bool bIsNoDelay = true) = 0;

	/**
	 * Joins this socket to the specified multicast group.
	 *
	 * The multicast group address must be in the range 224.0.0.0 to 239.255.255.255.
	 *
	 * @param GroupAddress The IP address of the multicast group.
	 * @return true on success, false otherwise.
	 * @see LeaveMulticastGroup, SetMulticastLoopback, SetMulticastTtl
	 */
	virtual bool JoinMulticastGroup(const FInternetAddr& GroupAddress) = 0;

	/**
	 * Joins this socket to the specified multicast group on the specified interface.
	 *
	 * The multicast group address must be in the range 224.0.0.0 to 239.255.255.255.
	 *
	 * @param GroupAddress The IP address of the multicast group.
	 * @param InterfaceAddress The address representing the interface.
	 * @return true on success, false otherwise.
	 * @see LeaveMulticastGroup, SetMulticastLoopback, SetMulticastTtl
	 */
	virtual bool JoinMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress) = 0;

	/**
	 * Removes this UDP client from the specified multicast group.
	 *
	 * @param The multicast group address to leave.
	 * @return true on success, false otherwise.
	 * @see JoinMulticastGroup, SetMulticastLoopback, SetMulticastTtl
	 */
	virtual bool LeaveMulticastGroup(const FInternetAddr& GroupAddress) = 0;

	/**
	 * Removes this UDP client from the specified multicast group on the specified interface.
	 *
	 * @param GroupAddress The multicast group address to leave.
	 * @param InterfaceAddress The address representing the interface.
	 * @return true on success, false otherwise.
	 * @see JoinMulticastGroup, SetMulticastLoopback, SetMulticastTtl
	 */
	virtual bool LeaveMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress) = 0;

	/**
	 * Enables or disables multicast loopback on the socket (UDP only).
	 *
	 * This setting determines whether multicast datagrams are looped
	 * back to the sending socket. By default, multicast loopback is
	 * enabled. It must be enabled if more than one listener is present
	 * on a host.
	 *
	 * @param bLoopback Whether loopback should be enabled.
	 * @see LeaveMulticastGroup, JoinMulticastGroup, SetMulticastTtl
	 */
	virtual bool SetMulticastLoopback(bool bLoopback) = 0;

	/**
	 * Sets the time to live (TTL) for multicast datagrams.
	 *
	 * The default TTL for multicast datagrams is 1, which prevents them
	 * from being forwarded beyond the local subnet. Higher values will
	 * allow multicast datagrams to be sent into neighboring subnets, if
	 * multicast capable routers are present.
	 *
	 * @param TimeToLive Number of hops the datagram can make.
	 * @see LeaveMulticastGroup, JoinMulticastGroup, SetMulticastLoopback
	 */
	virtual bool SetMulticastTtl(uint8 TimeToLive) = 0;

	/**
	 * Sets the interface used to send outgoing multicast datagrams.
	 *
	 * Multicast traffic is sent using the default interface, this allows 
	 * to explicitly set the interface used to send outgoing multicast datagrams.
	 *
	 * @param InterfaceAddress The interface address.
	 * @return true if the call succeeded, false otherwise.
	 */
	virtual bool SetMulticastInterface(const FInternetAddr& InterfaceAddress) = 0;

	/**
	 * Sets whether a socket can be bound to an address in use.
	 *
	 * @param bAllowReuse Whether to allow reuse or not.
	 * @return true if the call succeeded, false otherwise.
	 */
	virtual bool SetReuseAddr(bool bAllowReuse = true) = 0;

	/**
	 * Sets whether and how long a socket will linger after closing.
	 *
	 * @param bShouldLinger Whether to have the socket remain open for a time period after closing or not.
	 * @param Timeout The amount of time to linger before closing.
	 * @return true if the call succeeded, false otherwise.
	 */
	virtual bool SetLinger(bool bShouldLinger = true, int32 Timeout = 0) = 0;

	/**
	 * Enables error queue support for the socket.
	 *
	 * @param bUseErrorQueue Whether to enable error queuing or not.
	 * @return true if the call succeeded, false otherwise.
	 */
	virtual bool SetRecvErr(bool bUseErrorQueue = true) = 0;

	/**
	 * Sets the size of the send buffer to use.
	 *
	 * @param Size The size to change it to.
	 * @param NewSize Will contain the size that was set (in case OS can't set that).
	 * @return true if the call succeeded, false otherwise.
	 */
	virtual bool SetSendBufferSize(int32 Size, int32& NewSize) = 0;

	/**
	 * Sets the size of the receive buffer to use.
	 *
	 * @param Size The size to change it to.
	 * @param NewSize Will contain the size that was set (in case OS can't set that).
	 * @return true if the call succeeded, false otherwise.
	 */
	virtual bool SetReceiveBufferSize(int32 Size, int32& NewSize) = 0;

	/**
	 * Sets whether to retrieve the system-level receive timestamp, for sockets
	 *
	 * @param bRetrieveTimestamp	Whether to retrieve the timestamp upon receive
	 * @return						True if the call succeeded, false otherwise.
	 */
	SOCKETS_API virtual bool SetRetrieveTimestamp(bool bRetrieveTimestamp=true);

	/**
	 * Attempt to enable UDP segmentation offload (USO). Where supported, this
	 * reduces overhead by automatically splitting "superpackets" which exceed
	 * the specified segment size into multiple datagrams.
	 *
	 * @param InSegmentPayloadBytes Number of payload bytes for each split segment.
	 * @return True if the call succeeded, false otherwise.
	 */
	SOCKETS_API virtual bool SetSendSegmentationOffloadSize(uint16 InSegmentPayloadBytes);

	/**
	 * Reads the port this socket is bound to.
	 *
	 * @return Port number.
	 * @see GetSocketType, GetDescription
	 */ 
	virtual int32 GetPortNo() = 0;

	/**
	 * Sets whether to enable IP_PKTINFO support
	 *
	 * @param bEnable	Whether to enable/disable
	 * @return True if the call succeeded, false otherwise.
	 */
	SOCKETS_API virtual bool SetIpPktInfo(bool bEnable);

	/**
	 * Reads a chunk of data from the socket and gathers the source address and the destination using IP_PKTINFO
	 *
	 * A return value of 'true' does not necessarily mean that data was returned.
	 * Callers must check the 'BytesRead' parameter for the actual amount of data
	 * returned. A value of zero indicates that there was no data available for reading.
	 *
	 * @param Data The buffer to read into.
	 * @param BufferSize The max size of the buffer.
	 * @param BytesRead Will indicate how many bytes were read from the socket.
	 * @param Source Will contain the receiving the address of the sender of the data.
	 * @param Destination Will contain the destination address used by the sender of the data.
	 * @param Flags The receive flags.
	 * @return true on success, false in case of a closed socket or an unrecoverable error.
	 */
	SOCKETS_API virtual bool RecvFromWithPktInfo(uint8* Data, int32 BufferSize, int32& BytesRead, FInternetAddr& Source, FInternetAddr& Destination, ESocketReceiveFlags::Type Flags = ESocketReceiveFlags::None);

	/**
	 * Get the type of protocol the socket is bound to
	 *
	 * @return Socket type.
	 * @see GetDescription, GetPortNo
	 */
	inline ESocketType GetSocketType() const
	{
		return SocketType;
	}

	/**
	 * Get the debug description of the socket.
	 *
	 * @return Debug string.
	 * @see GetPortNo, GetSocketType
	 */
	inline FString GetDescription() const
	{
		return SocketDescription;
	}

	/**
	 * Get the type of protocol the socket is bound to
	 *
	 * @return Socket type.
	 * @see GetDescription, GetPortNo
	 */
	inline FName GetProtocol() const
	{
		return SocketProtocol;
	}
};
