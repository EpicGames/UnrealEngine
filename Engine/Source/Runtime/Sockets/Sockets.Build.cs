// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class Sockets : ModuleRules
{
	public Sockets(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("NetCommon");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"NetCommon"
			});

		PublicDefinitions.Add("SOCKETS_PACKAGE=1");

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
